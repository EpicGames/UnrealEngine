{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Procedural Content Generation Framework (PCG) Mesh Partition Interop",
	"Description": "Interoperability of Mesh Partition with PCG.",
	"Category": "Mesh Partition",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/latest/en-US/procedural-content-generation--framework-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PCGMeshPartitionInterop",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PCGMeshPartitionInteropEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PCG",
			"Enabled": true
		},
		{
			"Name": "MeshPartition",
			"Enabled": true
		},
		{
			"Name": "PCGGeometryScriptInterop",
			"Enabled": true
		}
	]
}