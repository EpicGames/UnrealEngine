// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "Data/Registry/PCGDataType.h"

#include "UObject/WeakObjectPtr.h"

#include "PCGMeshTerrainSectionData.generated.h"

class AActor;

namespace UE::MeshPartition
{

USTRUCT(meta = (PCG_DataTypeDisplayName = "Mesh Terrain Section"))
struct FPCGDataTypeInfoMeshTerrainSection : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO()
};

/** Reference to a single mesh terrain section. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshTerrainSectionData : public UPCGData
{
	GENERATED_BODY()

public:
	//~ Begin UPCGData interface
	PCG_ASSIGN_TYPE_INFO(FPCGDataTypeInfoMeshTerrainSection)
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	virtual UPCGData* DuplicateData(FPCGContext* Context, bool bInitializeMetadata = true) const override;
	//~ End UPCGData interface

	PCGMESHPARTITIONINTEROP_API AActor* GetSectionActor() const;
	void SetSectionActor(AActor* InSectionActor);

private:
	TWeakObjectPtr<AActor> SectionActor;
};

} // namespace UE::MeshPartition
