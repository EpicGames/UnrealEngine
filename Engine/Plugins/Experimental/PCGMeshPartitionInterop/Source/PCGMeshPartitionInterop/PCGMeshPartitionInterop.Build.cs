// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCGMeshPartitionInterop : ModuleRules
	{
		public PCGMeshPartitionInterop(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
			bAllowUETypesInNamespaces = true;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"Landscape",
					"PCG",
					"PCGCompute",
					"MeshPartition",
					"GeometryCore",
					"GeometryFramework",
					"PCGGeometryScriptInterop", // UPCGDynamicMeshData
					"RenderCore",
					"RHI",
					"Renderer"
				}
			);

			if (Target.bBuildEditor)
			{
				PublicDependencyModuleNames.AddRange(
					new string[] {
						"MeshPartitionEditor",
					}
				);
			}
		}
	}
}