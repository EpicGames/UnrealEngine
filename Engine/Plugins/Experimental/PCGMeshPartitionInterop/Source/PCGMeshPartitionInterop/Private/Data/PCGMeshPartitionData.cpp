// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGMeshPartitionData.h"

#include "Algo/Unique.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Elements/PCGMeshPartitionQuery.h"
#include "PCGComponent.h"
#include "PCGSubsystem.h"
#include "PCGContext.h"
#include "PCGPoint.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "MeshPartition.h"
#include "Engine/World.h"
#include "PCGMeshPartitionInteropModule.h"
#include "Serialization/ArchiveCrc32.h"
#include "Elements/PCGSurfaceSampler.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Async/ParallelFor.h"
#include "MeshPartitionPCGDataComponent.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "PCGMegaMeshQuery"

namespace UE::MeshPartition
{
namespace PCGMegaMeshDataLocals
{
	bool bRevertToDefaultSamplingToPointData = false;
	static FAutoConsoleVariableRef CVarRevertToDefaultSamplingForToPointData(
		TEXT("MegaMesh.PCG.RevertToDefaultSamplingToPointData"),
		bRevertToDefaultSamplingToPointData,
		TEXT("When true, reruns the modifier whenever the contained pcg graph changes."));
}

namespace MegaMeshQueryConstants
{
	const FName ImpactAttribute = TEXT("ImpactResult");
	const FName ImpactPointAttribute = TEXT("ImpactPoint");
	const FName ImpactNormalAttribute = TEXT("ImpactNormal");
	const FName ImpactDistanceAttribute = TEXT("ImpactDistance");
	const FName PhysicalMaterialReferenceAttribute = TEXT("PhysicalMaterial");
	const FName RenderMaterialReferenceAttribute = TEXT("ImpactRenderMaterial");
	const FName StaticMeshReferenceAttribute = TEXT("ImpactStaticMesh");
	const FName ElementIndexAttribute = TEXT("ImpactElementIndex");
	const FName UVCoordAttribute = TEXT("ImpactUVCoords");
	const FName FaceIndexAttribute = TEXT("ImpactFaceIndex");
}

namespace MegaMeshQueryHelpers
{
	FTransform GetOrthonormalImpactTransform(const FVector3d& HitPoint, const FVector3d& HitNormal)
	{
		// Implementation note: this uses the same orthonormalization process as the world ray query
		ensure(HitNormal.IsNormalized());
		const FVector ArbitraryVector = (FMath::Abs(HitNormal.Y) < (1.f - UE_KINDA_SMALL_NUMBER) ? FVector::YAxisVector : FVector::ZAxisVector);
		const FVector XAxis = (ArbitraryVector ^ HitNormal).GetSafeNormal();
		const FVector YAxis = (HitNormal ^ XAxis);

		return FTransform(XAxis, YAxis, HitNormal, HitPoint);
	}

	bool CreateHitAttributes(const MeshPartition::FPCGQueryParams& InQueryParams, UPCGMetadata* OutMetadata)
	{
		if (!OutMetadata)
		{
			return false;
		}

		auto CreateAttribute = [OutMetadata]<typename Type>(FName AttributeName, bool bShouldCreate, const Type& DefaultValue)
		{
			if (!bShouldCreate)
			{
				return true;
			}

			if (!OutMetadata->HasAttribute(AttributeName))
			{
				if (OutMetadata->CreateAttribute<Type>(AttributeName, DefaultValue, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false))
				{
					return true;
				}
			}

			return false;
		};

		bool bResult = true;
		bResult &= CreateAttribute(MegaMeshQueryConstants::ImpactPointAttribute, InQueryParams.bGetImpactPoint, FVector::ZeroVector);
		bResult &= CreateAttribute(MegaMeshQueryConstants::ImpactNormalAttribute, InQueryParams.bGetImpactNormal, FVector::ZeroVector);
		bResult &= CreateAttribute(MegaMeshQueryConstants::ImpactDistanceAttribute, InQueryParams.bGetDistance, 0.0);
		bResult &= CreateAttribute(MegaMeshQueryConstants::FaceIndexAttribute, InQueryParams.bGetFaceIndex, int32{0});
		bResult &= CreateAttribute(MegaMeshQueryConstants::UVCoordAttribute, InQueryParams.bGetUVCoords, FVector2d::ZeroVector);

		for (const FName& Channel : InQueryParams.Channels)
		{
			if (OutMetadata->HasAttribute(Channel))
			{
				continue;
			}

			OutMetadata->CreateFloatAttribute(Channel, 0.f, true, true);
		}

		return bResult;
	}

	bool ApplyHitMetadata(
		const FVector3d& WorldHitPoint,
		const FVector3d& WorldHitNormal,
		int32 Tid,
		double DistanceAlongHitRay,
		const MeshPartition::FPCGQueryParams& InQueryParams,
		FPCGPoint& OutPoint,
		UPCGMetadata* OutMetadata,
		bool bInShouldCreateAttributes)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MegaMeshQueryHelpers::ApplyHitMetadata);

		if (!OutMetadata)
		{
			return false;
		}

		if (bInShouldCreateAttributes)
		{
			CreateHitAttributes(InQueryParams, OutMetadata);
		}

		auto ApplyAttribute = [&OutPoint, OutMetadata]<typename Type>(FName AttributeName, const Type& Value, bool bShouldApply = true)
		{
			if (!bShouldApply)
			{
				return true;
			}

			if (FPCGMetadataAttribute<Type>* Attribute = OutMetadata->GetMutableTypedAttribute<Type>(AttributeName))
			{
				OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);
				Attribute->SetValue(OutPoint.MetadataEntry, Value);
				return true;
			}

			return false;
		};

		bool bResult = true;
		// Note: The T/F Impact attribute is true by default, so no need to set it directly.
		bResult &= ApplyAttribute(MegaMeshQueryConstants::ImpactPointAttribute, WorldHitPoint, InQueryParams.bGetImpactPoint);
		bResult &= ApplyAttribute(MegaMeshQueryConstants::ImpactNormalAttribute, WorldHitNormal, InQueryParams.bGetImpactNormal);
		bResult &= ApplyAttribute(MegaMeshQueryConstants::ImpactDistanceAttribute, DistanceAlongHitRay, InQueryParams.bGetDistance);
		bResult &= ApplyAttribute(MegaMeshQueryConstants::FaceIndexAttribute, Tid, InQueryParams.bGetFaceIndex);

		return bResult;
	}

	bool ApplyHitChannelsMetadata(const FMeshData* InMesh, int32 Tid, FVector3f BaryCoords,
		const MeshPartition::FPCGQueryParams& InQueryParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MegaMeshQueryHelpers::ApplyHitChannelsMetadata);

		if (!InMesh || !OutMetadata)
		{
			return false;
		}

		const Geometry::FIndex3i TriVids = InMesh->GetTriangle(Tid);

		bool bSetUVs = false;
		if (InQueryParams.bGetUVCoords)
		{
			if (FPCGMetadataAttribute<FVector2d>* Attribute = OutMetadata->GetMutableTypedAttribute<FVector2d>(MegaMeshQueryConstants::UVCoordAttribute))
			{
				FVector2f UVValue = FVector2f::Zero();

				for (int VertIndex = 0; VertIndex < 3; ++VertIndex)
				{
					// #TODO: how should we expose uv layers to the pcg graph? For now just give access to uv 0
					const FVector2f VertUV = InMesh->GetVertexUV(TriVids[VertIndex], 0);
					UVValue += VertUV * BaryCoords[VertIndex];
				}
				
				OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);
				Attribute->SetValue(OutPoint.MetadataEntry, FVector2d(UVValue));
				bSetUVs = true;
			}
		}

		if (InQueryParams.Channels.IsEmpty())
		{
			return bSetUVs;
		}

		if (!bSetUVs)
		{
			OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);
		}

		for (const FName& WeightLayerName : InMesh->GetWeightLayerNames())
		{
			if (InQueryParams.Channels.Find(WeightLayerName) == INDEX_NONE)
			{
				continue;
			}

			FPCGMetadataAttribute<float>* Attribute = OutMetadata->GetMutableTypedAttribute<float>(WeightLayerName);
			if (!Attribute)
			{
				continue;
			}

			FVector3f TriValues;
			for (int i = 0; i < 3; ++i)
			{
				TriValues[i] = InMesh->GetWeightLayerValue(WeightLayerName, TriVids[i]);
			}

			Attribute->SetValue(OutPoint.MetadataEntry, BaryCoords.Dot(TriValues));
		}

		return true;
	}
}

void UPCGMeshPartitionData::Initialize(FPCGMeshPartitionElementContext* InContext, UWorld* InWorld, const FTransform& InTransform, const FBox& InBounds, const FBox& InLocalBounds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UPCGMeshPartitionData::Initialize);
	World = InWorld;
	Transform = InTransform;
	Bounds = InBounds;
	LocalBounds = InLocalBounds;

	SectionDatas.SetNum(InContext->SectionDatas.Num());
	ParallelFor(InContext->SectionDatas.Num(), [&](int Index)
		{
			FSectionData& SectionData = SectionDatas[Index];
			SectionData.MegaMeshActor = InContext->SectionDatas[Index].MegaMeshActor;

			if (MeshPartition::UPCGDataComponent* PCGData = InContext->SectionDatas[Index].PCGData.Get())
			{
				SectionData.MeshData = PCGData->GetMesh();
				SectionData.Transform = PCGData->GetOwner()->GetActorTransform();

				SectionData.BuildSpatialTask = Tasks::Launch(
					TEXT("WaitSpatialDataBuild"),
					[&SectionData, PCGData]
					{
						SectionData.Spatial = PCGData->GetSpatial();
					},
					Tasks::Prerequisites(PCGData->GetSpatialBuildTask())
				);
			}
#if WITH_EDITOR
			else if (InContext->SectionDatas[Index].MeshData.IsValid())
			{
				SectionData.MeshData = InContext->SectionDatas[Index].MeshData;
				SectionData.Spatial = InContext->SectionDatas[Index].Spatial;
				SectionData.Transform = InContext->SectionDatas[Index].MeshDataTransform;
			}
#endif // WITH_EDITOR
		});
}

bool UPCGMeshPartitionData::IsDataReady() const
{
	for (const FSectionData& SectionData : SectionDatas)
	{
		if (SectionData.BuildSpatialTask.IsValid() && !SectionData.BuildSpatialTask.IsCompleted())
		{
			return false;
		}
	}
	return true;
}

void UPCGMeshPartitionData::AddToCrc(FArchiveCrc32& InAr, bool bInFullDataCrc) const
{
	Super::AddToCrc(InAr, bInFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(InAr);

	// Note: if we someday implement a persistable crc, we'll need to invalidate when the cvar changes. Though
	//  perhaps we will remove this reversion option by that point.
	bool bDefaultSamplingCVarState = PCGMegaMeshDataLocals::bRevertToDefaultSamplingToPointData;
	InAr << bDefaultSamplingCVarState;
}

bool UPCGMeshPartitionData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UPCGMeshPartitionData::SamplePoint);

	check(World.IsValid());

	const FVector RayOrigin = InTransform.GetLocation() - ((InTransform.GetLocation() - QueryParams.RayOrigin) | QueryParams.RayDirection) * QueryParams.RayDirection;

	// It's possible to hit multiple sections with one ray, so we have to try all of them and keep
	//  the closest one (unless the user settings allows us to grab the first one we hit, see below)
	double BestHitDistance = TNumericLimits<double>::Max();
	int32 BestHitTid = IndexConstants::InvalidID;
	FVector3d BestHitBaryCoords;
	FVector3d BestHitPointLocal;
	FVector3d BestHitPointWorld;
	int32 BestHitSectionIndex = IndexConstants::InvalidID;

	for (int32 SectionIndex = 0; SectionIndex < SectionDatas.Num(); ++SectionIndex)
	{
		const FSectionData& SectionData = SectionDatas[SectionIndex];
		const FRay3d WorldRay(RayOrigin, QueryParams.RayDirection);
		
		FRay3d LocalRay(SectionData.Transform.InverseTransformPosition(WorldRay.Origin),
			SectionData.Transform.InverseTransformVector(WorldRay.Direction), false /*bDirectionIsNormalized*/);

		if (!ensure(SectionData.Spatial && SectionData.Spatial->IsValid(/* bAllowUnsafeModifiedMeshQueries */ false)))
		{
			continue;
		}

		double HitTValue;
		int32 HitTid;
		FVector3d BaryCoords;
		if (!SectionData.Spatial->FindNearestHitTriangle(LocalRay, HitTValue, HitTid, BaryCoords))
		{
			continue;
		}

		// It's likely that section scales don't differ, so we could just compare HitTValue, but we'll be safe
		//  and compare proper world locations.
		FVector3d HitPoint = LocalRay.PointAt(HitTValue);
		FVector3d WorldHitPoint = SectionData.Transform.TransformPosition(HitPoint);
		double Distance = (WorldHitPoint - RayOrigin).Length();

		if (Distance < BestHitDistance)
		{
			BestHitDistance = Distance;
			BestHitTid = HitTid;
			BestHitBaryCoords = BaryCoords;
			BestHitPointLocal = HitPoint;
			BestHitPointWorld = WorldHitPoint;
			BestHitSectionIndex = SectionIndex;
		
			if (QueryParams.bAcceptAnyHitSection)
			{
				break;
			}
		}
	}

	if (BestHitTid != IndexConstants::InvalidID && ensure(SectionDatas.IsValidIndex(BestHitSectionIndex)))
	{
		const FSectionData& SectionData = SectionDatas[BestHitSectionIndex];
		const FMeshData* Mesh = SectionData.MeshData.Get();
		const Geometry::FTransformSRT3d BaseTransform(SectionData.Transform);
		FVector3d WorldHitNormal = BaseTransform.TransformNormal(Mesh->GetTriNormal(BestHitTid));

		OutPoint = FPCGPoint(MegaMeshQueryHelpers::GetOrthonormalImpactTransform(BestHitPointWorld, WorldHitNormal), 1.0f,
			// Uncertain whether to use world position or local position here. Local position means that translating the mesh
			//  and the sampler at the same time will keep results the same. On the other hand it means that seeds could repeat
			//  across instances of the megamesh scattered through the world, and doing a shift of the mesh under the sampler
			//  without actually changing the hit locations (e.g. translating a plane) gives different results.
			PCGHelpers::ComputeSeedFromPosition(BestHitPointLocal));

		if (OutMetadata)
		{
			MegaMeshQueryHelpers::ApplyHitMetadata(BestHitPointWorld, WorldHitNormal, BestHitTid, BestHitDistance, 
				QueryParams, OutPoint, OutMetadata, /* bShouldCreateAttributes */true);
			MegaMeshQueryHelpers::ApplyHitChannelsMetadata(Mesh, BestHitTid, FVector3f(BestHitBaryCoords), QueryParams, OutPoint, OutMetadata);
		}

		return true;
	}

	return false;
}

const UPCGPointArrayData* UPCGMeshPartitionData::CreatePointArrayData(FPCGContext* InContext, const FBox& InBounds) const
{
	return Cast<UPCGPointArrayData>(CreateBasePointData(InContext, InBounds, UPCGPointArrayData::StaticClass()));
}

const UPCGPointData* UPCGMeshPartitionData::CreatePointData(FPCGContext* InContext, const FBox& InBounds) const
{
	return Cast<UPCGPointData>(CreateBasePointData(InContext, InBounds, UPCGPointData::StaticClass()));
}
const UPCGBasePointData* UPCGMeshPartitionData::CreateBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	using namespace PCGMegaMeshDataLocals;

	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	Data->InitializeFromData(this);

	FBox EffectiveBounds = Bounds;
	if (InBounds.IsValid)
	{
		if (Bounds.IsValid)
		{
			EffectiveBounds = Bounds.Overlap(InBounds);
		}
		else
		{
			EffectiveBounds = InBounds;
		}
	}


	// Early out
	if (!EffectiveBounds.IsValid)
	{
		return Data;
	}

	if (bRevertToDefaultSamplingToPointData)
	{
		// The default params will be fine in this case
		const PCGSurfaceSampler::FSurfaceSamplerParams Params;
		Data = PCGSurfaceSampler::SampleSurface(Context, Params, /*InSurface=*/this, /*InBoundingShape=*/nullptr, EffectiveBounds, PointDataClass);
		return Data;
	}

	// Make sure the attributes are unique and created on our output data
	TArray<FName> WeightLayerNames = QueryParams.Channels;
	WeightLayerNames.SetNum(Algo::Unique(WeightLayerNames));

	UPCGMetadata* PointMetadata = Data->MutableMetadata();
	for (int32 i = 0; i < WeightLayerNames.Num(); ++i)
	{
		if (!ensure(PCGMetadataElementCommon::ClearOrCreateAttribute<float>(PointMetadata, WeightLayerNames[i])))
		{
			WeightLayerNames.RemoveAtSwap(i);
			--i;
		}
	}

	// Used to track our index in the overall set of points from all sections
	int32 PointIndex = 0;

	for (int32 SectionIndex = 0; SectionIndex < SectionDatas.Num(); ++SectionIndex)
	{
		const FSectionData& SectionData = SectionDatas[SectionIndex];
		if (!SectionData.MeshData)
		{
			continue;
		}

		int32 SectionStartIndex = PointIndex;
		
		// Gether the data for this section
		TArray<FVector3d> PointWorldPositions;
		TArray<FVector3d> PointWorldNormals;
		TMap<FName, TArray<float>> WeightValues; // Layer name to values for that layer
		TArray<FName> SectionWeightLayerNames = SectionData.MeshData->GetWeightLayerNames();

		// Use this class for the normal since it has a proper TransformNormal
		UE::Geometry::FTransformSRT3d SectionTransform(SectionData.Transform);

		for (int32 Vid : SectionData.MeshData->VertexIndicesItr())
		{
			FVector3d PointWorldPosition = SectionData.Transform.TransformPosition(SectionData.MeshData->GetVertex(Vid));
			if (!EffectiveBounds.IsInside(PointWorldPosition))
			{
				continue;
			}

			PointWorldPositions.Add(PointWorldPosition);

			PointWorldNormals.Add(SectionTransform.TransformNormal(FVector(SectionData.MeshData->GetVertexNormal(Vid))));

			for (const FName& WeightLayer : SectionWeightLayerNames)
			{
				TArray<float>& Values = WeightValues.FindOrAdd(WeightLayer);
				const float Value = SectionData.MeshData->GetWeightLayerValue(WeightLayer, Vid);
				Values.Add(Value);
			}

			++PointIndex;
		}//end processing verts

		ensure(PointWorldPositions.Num() == PointWorldNormals.Num());

		if (PointWorldPositions.IsEmpty())
		{
			// No verts in this section were inside the bound
			continue;
		}

		// Now set the data for this section
		Data->SetNumPoints(PointIndex, /*bInitializeValues*/ false);
		TPCGValueRange<FTransform> TransformRange = Data->GetTransformValueRange();
		TPCGValueRange<int32> SeedRange = Data->GetSeedValueRange();
		TPCGValueRange<int64> MetadataEntryRange = Data->GetMetadataEntryValueRange();
		
		for (int32 VertIndex = 0; VertIndex < PointWorldPositions.Num(); ++VertIndex)
		{
			int32 OverallIndex = SectionStartIndex + VertIndex;
			TransformRange[OverallIndex] = MegaMeshQueryHelpers::GetOrthonormalImpactTransform(PointWorldPositions[VertIndex], PointWorldNormals[VertIndex]);
			SeedRange[OverallIndex] = PCGHelpers::ComputeSeedFromPosition(PointWorldPositions[VertIndex]);
			MetadataEntryRange[OverallIndex] = PCGInvalidEntryKey;

		}
		
		// Apply the weight data for this mesh
		FPCGAttributePropertyInputSelector Selector;
		for (const FName& LayerName : WeightLayerNames)
		{
			Selector.Update(LayerName.ToString());
			TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(Data, Selector);
			TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(Data, Selector);

			if (!ensure(Accessor && Keys))
			{
				continue;
			}

			TArray<float>* Values = WeightValues.Find(LayerName);
			bool bSetSuccessful = false;
			if (Values)
			{
				bSetSuccessful = Accessor->SetRange<float>(*Values, SectionStartIndex, *Keys);
			}
			else
			{
				// This must be a layer name that wasn't present in this section. We need to initialize it with zeroes
				//  for the points that came from this section.
				TArray<float> Zeros;
				int32 SectionNumContainedVertices = PointIndex - SectionStartIndex;
				Zeros.SetNumZeroed(SectionNumContainedVertices);
				bSetSuccessful = Accessor->SetRange<float>(Zeros, SectionStartIndex, *Keys);
			}

			ensure(bSetSuccessful);
		}

		// TODO: Should different sections get some index identifier so that the points can be sorted by section?
		//  It would be easy to add here using SectionIndex.

	}//end going through sections

	

	return Data;
}

UPCGSpatialData* UPCGMeshPartitionData::CopyInternal(FPCGContext* InContext) const
{
	MeshPartition::UPCGMeshPartitionData* NewData = FPCGContext::NewObject_AnyThread<MeshPartition::UPCGMeshPartitionData>(InContext);

	CopyBaseSurfaceData(NewData);

	NewData->World = World;
	NewData->OriginatingComponent = OriginatingComponent;
	NewData->Bounds = Bounds;
	NewData->QueryParams = QueryParams;
	NewData->SectionDatas = SectionDatas;

	return NewData;
}
}

#undef LOCTEXT_NAMESPACE