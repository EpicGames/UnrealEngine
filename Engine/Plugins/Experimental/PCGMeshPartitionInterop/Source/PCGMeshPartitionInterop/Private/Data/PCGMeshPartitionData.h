// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Data/PCGSurfaceData.h"
#include "MeshPartitionMeshData.h"
#include "Tasks/Task.h"

#include "PCGMeshPartitionData.generated.h"

namespace UE::MeshPartition
{
class AMeshPartition;
struct FPCGMeshPartitionElementContext;

UENUM(BlueprintType, Meta=(DisplayName = "Mesh Partition PCG Query Type"))
enum class EPCGQueryType : uint8
{
	/** Samples against the Mega Mesh Base modifiers directly before any procedural applications. Not valid at runtime */
	Base,
	/** Samples the Mega Mega at intermediate build layer and sub priority. Not valid at runtime */
	Intermediate,
	/** Samples the Mega Mega at intermediate build layers, not considering sub priority. Not valid at runtime */
	IntermediateLayer,
	/** Samples against the final built Mega Mesh (after all procedural application). Valid at runtime */
	Final,
};

USTRUCT(BlueprintType, Meta = (DisplayName = "Mesh Partition PCG Query Params"))
struct FPCGQueryParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, EditCondition = "bOverrideDefaultParams", EditConditionHides))
	FVector RayOrigin = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, EditCondition = "bOverrideDefaultParams", EditConditionHides))
	FVector RayDirection = FVector(0.0, 0.0, -1.0);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, EditCondition = "bOverrideDefaultParams", EditConditionHides))
	double RayLength = 1.0e+7;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable))
	MeshPartition::EPCGQueryType QueryType = MeshPartition::EPCGQueryType::Base;

	/** When sampling an intermediate build, the layer up to which to build the mesh (not including the layer itself, unless Inlcusive is set). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable,
		EditCondition = "QueryType == EPCGQueryType::Intermediate || QueryType == EPCGQueryType::IntermediateLayer", EditConditionHides))
	FName LayerName;

	/** When sampling an intermediate build, optional specific sub-priority up to which to build the mesh within the chosen priority layer (higher priorities are applied after lower). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, 
		EditCondition = "QueryType == EPCGQueryType::Intermediate", EditConditionHides))
	double SubPriority = 0;

	/** When sampling an intermediate build, whether we build up to but not including the given layer/sub-priority, or whether we include it. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, 
		EditCondition = "QueryType == EPCGQueryType::Intermediate || QueryType == EPCGQueryType::IntermediateLayer", EditConditionHides))
	bool bInclusive = false;

	/** When not sampling base sections, whether to update the vertex normals. Vertex normals are used for point
	rotations when converting to points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, AdvancedDisplay, meta = (PCG_Overridable,
		EditCondition = "QueryType != EPCGQueryType::Base", EditConditionHides))
	uint8 bRecomputeVertexNormals : 1 = false;

	/** Set ray parameters including origin, direction and length explicitly rather than deriving these from the generating actor bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (DisplayName = "Set Ray Parameters", PCG_Overridable))
	uint8 bOverrideDefaultParams : 1 = false;

	/** Specifies which channel we will want to sample. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	TArray<FName> Channels;

	/** Create an attribute for the impact location in world space. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetImpactPoint : 1 = false;

	/** Create an attribute for the impact normal. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetImpactNormal : 1 = false;

	/** Create an attribute for the distance between the ray origin and the impact point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetDistance : 1 = false;

	/** Create an attribute for index of the hit face. Note: Will only work in complex traces. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetFaceIndex : 1 = false;

	/** Create an attribute for UV Coordinates of the surface hit. Note: Will only work in complex traces. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetUVCoords : 1 = false;

	/** When false, raycast all possible sections to make sure we get the closest hit, but if true, accept any hit.
	This can be left true to speed up sampling, as long as your sections don't overlap in such a way that a sampling
	ray could hit multiple sections. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, AdvancedDisplay, meta = (PCG_Overridable))
	uint8 bAcceptAnyHitSection : 1 = true;

	/** When supplied, only the given megamesh will be queried. Otherwise, all megameshes in bounds will be queried. */
	UPROPERTY(EditAnywhere, Category = Settings, Meta = (PCG_Overridable))
	TSoftObjectPtr<AMeshPartition> MegaMeshOverride;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshPartitionData : public UPCGSurfaceData
{
	GENERATED_BODY()

public:
	void Initialize(FPCGMeshPartitionElementContext* InContext, UWorld* InWorld, const FTransform& InTransform, const FBox& InBounds = FBox(EForceInit::ForceInit), const FBox& InLocalBounds = FBox(EForceInit::ForceInit));
	bool IsDataReady() const;

	// ~Begin UPCGData interface
	virtual void AddToCrc(FArchiveCrc32& InAr, bool bInFullDataCrc) const override;
	// ~End UPCGData interface

	//~Begin UPCGSpatialData interface
	virtual FBox GetBounds() const override { return Bounds; }
	virtual FBox GetStrictBounds() const override { return Bounds; }
	virtual bool IsBounded() const override { return !!Bounds.IsValid; }
	virtual bool SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	virtual bool HasNonTrivialTransform() const override { return true; }
	virtual FVector GetNormal() const override { return Transform.GetRotation().GetUpVector(); }
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	// ~Begin UPCGSpatialDataWithPointCache interface
	virtual bool SupportsBoundedPointData() const { return true; }
	virtual const UPCGPointData* CreatePointData(FPCGContext* InContext) const override { return CreatePointData(InContext, FBox(EForceInit::ForceInit)); }
	virtual const UPCGPointData* CreatePointData(FPCGContext* InContext, const FBox& InBounds) const override;
	virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* InContext, const FBox& InBounds) const override;
	// ~End UPCGSpatialDataWithPointCache interface

	UPROPERTY()
	TWeakObjectPtr<UWorld> World = nullptr;

	UPROPERTY()
	TWeakObjectPtr<UPCGComponent> OriginatingComponent = nullptr;

	UPROPERTY()
	FBox Bounds = FBox(EForceInit::ForceInit);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (ShowOnlyInnerProperties))
	MeshPartition::FPCGQueryParams QueryParams;

	struct FSectionData
	{
		TSharedPtr<const FMeshData> MeshData;

		FTransform Transform;

		TSharedPtr<const FMeshABBTree3> Spatial;
		
		Tasks::FTask BuildSpatialTask;

		TWeakObjectPtr<AMeshPartition> MegaMeshActor;
	};
	TArray<FSectionData> SectionDatas;

private:
	const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const;
};
}