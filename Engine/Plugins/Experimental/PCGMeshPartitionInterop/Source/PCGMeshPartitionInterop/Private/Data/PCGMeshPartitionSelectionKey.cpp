// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGMeshPartitionSelectionKey.h"
#include "PCGMeshPartitionData.h"

namespace UE::MeshPartition
{
FPCGLayerSelectionKey::FPCGLayerSelectionKey(MeshPartition::EPCGQueryType InQueryType, FName InLayerName,
	double InSubPriority, bool bInIsFromPrevious, EPCGLayerSelectionKeyType InKeyType)
	: QueryType(InQueryType)
	, LayerName(InLayerName)
	, SubPriority(InSubPriority)
	, bIsFromPrevious(bInIsFromPrevious)
	, KeyType(InKeyType)
{
}

bool FPCGLayerSelectionKey::IsMatchingInternal(const FPCGCustomSelectionKey& InCustomKey) const
{
	const MeshPartition::FPCGLayerSelectionKey* OtherKey = InCustomKey.CastTo<MeshPartition::FPCGLayerSelectionKey>();
	check(OtherKey);

	// Our comparison is not symmetric, because the SubPriority of the listener key needs
	//  to be higher than the SubPriority of the change key, and bInclusive has
	//  different meaning for the two key types. However if we're not in the situation
	//  of comparing a listener key to a change key, we revert to a simple comparison of
	//  values.
	if (KeyType == OtherKey->KeyType)
	{
		if (QueryType != OtherKey->QueryType)
		{
			return false;
		}

		if (QueryType == MeshPartition::EPCGQueryType::IntermediateLayer)
		{
			return LayerName == OtherKey->LayerName
				&& bIsFromPrevious == OtherKey->bIsFromPrevious;
		}
		if (QueryType == MeshPartition::EPCGQueryType::Intermediate)
		{
			return LayerName == OtherKey->LayerName
				&& bIsFromPrevious == OtherKey->bIsFromPrevious
				&& SubPriority == OtherKey->SubPriority;
		}
		else
		{
			return true;
		}
	}

	// If we got here, one is a listener key and the other is not
	const MeshPartition::FPCGLayerSelectionKey* ListenerKey = this;
	if (KeyType != EPCGLayerSelectionKeyType::Listener)
	{
		Swap(ListenerKey, OtherKey);
	}

	switch (ListenerKey->QueryType)
	{
	case MeshPartition::EPCGQueryType::Base:
		return OtherKey->QueryType == MeshPartition::EPCGQueryType::Base;
	case MeshPartition::EPCGQueryType::Final:
		return OtherKey->QueryType == MeshPartition::EPCGQueryType::Final;
	case MeshPartition::EPCGQueryType::IntermediateLayer:
		return (OtherKey->QueryType == MeshPartition::EPCGQueryType::Intermediate
			|| OtherKey->QueryType == MeshPartition::EPCGQueryType::IntermediateLayer)
			&& OtherKey->LayerName == ListenerKey->LayerName
			// Either the change is from a previous layer, or we don't care that it's not (i.e. inclusive)
			&& (OtherKey->bIsFromPrevious || !ListenerKey->bIsFromPrevious);
	case MeshPartition::EPCGQueryType::Intermediate:
		return (OtherKey->QueryType == MeshPartition::EPCGQueryType::Intermediate
			|| OtherKey->QueryType == MeshPartition::EPCGQueryType::IntermediateLayer)
			&& OtherKey->LayerName == ListenerKey->LayerName
			// Either the change is from a previous layer..
			&& (OtherKey->bIsFromPrevious
				// Or it is from a lower priority modifier...
				|| OtherKey->SubPriority < ListenerKey->SubPriority
				// Or priority matches and we are ok with that
				|| (OtherKey->SubPriority == ListenerKey->SubPriority && !ListenerKey->bIsFromPrevious));
	default:
		return ensure(false);
	}
}

bool FPCGLayerSelectionKey::EqualsInternal(const FPCGCustomSelectionKey& InCustomKey) const
{
	const MeshPartition::FPCGLayerSelectionKey* OtherKey = InCustomKey.CastTo<MeshPartition::FPCGLayerSelectionKey>();
	check(OtherKey);
	return QueryType == OtherKey->QueryType 
		&& LayerName == OtherKey->LayerName
		&& SubPriority == OtherKey->SubPriority
		&& bIsFromPrevious == OtherKey->bIsFromPrevious
		&& KeyType == OtherKey->KeyType;
}

uint32 FPCGLayerSelectionKey::GetTypeHashInternal() const
{
	uint32 Hash = HashCombine(GetTypeHash(QueryType), GetTypeHash(LayerName));
	Hash = HashCombine(Hash, GetTypeHash(SubPriority));
	Hash = HashCombine(Hash, GetTypeHash(bIsFromPrevious));
	Hash = HashCombine(Hash, GetTypeHash(KeyType));
	return Hash;
}

bool FPCGGlobalSelectionKey::IsMatchingInternal(const FPCGCustomSelectionKey& InCustomKey) const
{
	const MeshPartition::FPCGGlobalSelectionKey* OtherKey = InCustomKey.CastTo<MeshPartition::FPCGGlobalSelectionKey>();
	check(OtherKey);

	return true;
}

bool FPCGGlobalSelectionKey::EqualsInternal(const FPCGCustomSelectionKey& InCustomKey) const
{
	const MeshPartition::FPCGGlobalSelectionKey* OtherKey = InCustomKey.CastTo<MeshPartition::FPCGGlobalSelectionKey>();
	check(OtherKey);
	
	return true;
}

uint32 FPCGGlobalSelectionKey::GetTypeHashInternal() const
{
	static uint32 Hash = GetTypeHash(MeshPartition::FPCGGlobalSelectionKey::StaticStruct()->GetFName());
	return Hash;
}
}