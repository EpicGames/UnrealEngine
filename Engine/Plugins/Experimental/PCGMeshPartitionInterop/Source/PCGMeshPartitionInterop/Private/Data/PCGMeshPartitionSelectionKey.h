// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Elements/PCGActorSelector.h"
#include "Data/PCGMeshPartitionData.h"

#include "PCGMeshPartitionSelectionKey.generated.h"

namespace UE::MeshPartition
{
enum class EPCGQueryType : uint8;

/**
 * To control how the key is matched against other keys, keys are marked with their
 *  context for use. For example, the priority on a change key would match a listener
 *  key if the listener priority is higher, but not the reverse.
 */
UENUM()
enum class EPCGLayerSelectionKeyType
{
	// This key is used to listen to change events
	Listener,
	// This key is issued with change events
	ChangeNotification
};

/**
 * Selection Key used to listen to a mesh partition change. Mesh partition changes trigger
 *  selection key notifications corresponding to layers at the changed layer and above.
 */
USTRUCT()
struct FPCGLayerSelectionKey : public FPCGCustomSelectionKey
{
public:
	GENERATED_BODY()

	FPCGLayerSelectionKey() = default;
	explicit FPCGLayerSelectionKey(MeshPartition::EPCGQueryType InQueryType, 
		FName InLayerName = NAME_None, 
		double InSubPriority = 0,
		bool bInIsFromPrevious = false,
		EPCGLayerSelectionKeyType InKeyType = EPCGLayerSelectionKeyType::ChangeNotification);

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FPCGLayerSelectionKey::StaticStruct();
	}

protected:

	UPROPERTY()
	MeshPartition::EPCGQueryType QueryType = MeshPartition::EPCGQueryType::Intermediate;

	/**
	 * Only relevant when QueryType is Intermediate or IntermediateLayer. Any change will also
	 *  trigger a change with a None layer name, to allow matching against layers that are not
	 *  found in the definition.
	 */
	UPROPERTY()
	FName LayerName = NAME_None;

	/**
	 * Only relevant when QueryType is Intermediate.
	 */
	UPROPERTY()
	double SubPriority = 0;

	/**
	 * For change notifications, this is set to true if the change was made in a preceding layer.
	 * For query keys, this should be set to true if the change has to be made in a preceding
	 *  layer or priority (i.e. matching is exclusive)
	 */
	UPROPERTY()
	bool bIsFromPrevious = false;
	
	/**
	 * Marks this key as being a query (rather than a change notification), which determines
	 *  how SubPriority and bIsFromPrevious are interpreted when matching against other keys
	 *  (e.g. which SubPriority needs to be higher for a match).
	 */
	UPROPERTY()
	EPCGLayerSelectionKeyType KeyType = EPCGLayerSelectionKeyType::ChangeNotification;
	
	virtual bool IsMatchingInternal(const FPCGCustomSelectionKey& InCustomKey) const override;
	virtual bool EqualsInternal(const FPCGCustomSelectionKey& InCustomKey) const override;
	virtual uint32 GetTypeHashInternal() const override;
};

/**
* Selection Key used to listen to global changes.
*/
USTRUCT()
struct FPCGGlobalSelectionKey : public FPCGCustomSelectionKey
{
public:
	GENERATED_BODY()

	FPCGGlobalSelectionKey() = default;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FPCGGlobalSelectionKey::StaticStruct();
	}

protected:

	virtual bool IsMatchingInternal(const FPCGCustomSelectionKey& InCustomKey) const override;
	virtual bool EqualsInternal(const FPCGCustomSelectionKey& InCustomKey) const override;
	virtual uint32 GetTypeHashInternal() const override;
};
}