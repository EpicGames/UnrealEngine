// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGMeshTerrainSectionData.h"

#include "PCGContext.h"

#include "GameFramework/Actor.h"
#include "Serialization/ArchiveCrc32.h"

namespace UE::MeshPartition
{

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoMeshTerrainSection, UPCGMeshTerrainSectionData)

AActor* UPCGMeshTerrainSectionData::GetSectionActor() const
{
	return SectionActor.Get();
}

void UPCGMeshTerrainSectionData::SetSectionActor(AActor* InSectionActor)
{
	SectionActor = InSectionActor;
}

void UPCGMeshTerrainSectionData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

UPCGData* UPCGMeshTerrainSectionData::DuplicateData(FPCGContext* Context, bool bInitializeMetadata) const
{
	UPCGMeshTerrainSectionData* NewData = FPCGContext::NewObject_AnyThread<UPCGMeshTerrainSectionData>(Context);
	NewData->SectionActor = SectionActor;
	return NewData;
}

} // namespace UE::MeshPartition
