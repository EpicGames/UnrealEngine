// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGBakeMeshTerrainSectionMesh.h"

#include "PCGMeshPartitionInteropModule.h"
#include "PCGDilate.h"
#include "PCGModule.h"
#include "PCGUnwrapMesh.h"
#include "Data/PCGMeshTerrainSectionData.h"
#include "Data/PCGRenderTargetData.h"
#include "Utils/PCGLogErrors.h"

#include "MeshPartitionCompiledSection.h"

#if WITH_EDITOR
#include "MeshPartitionPreviewComponents.h"
#include "MeshPartitionPreviewSceneProxy.h"
#include "MeshPartitionPreviewSection.h"
#endif

#include "Engine/StaticMesh.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GameFramework/Actor.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "StaticMeshResources.h"
#include "TextureResource.h"

#define LOCTEXT_NAMESPACE "PCGBakeMeshTerrainSectionMeshElement"

namespace UE::MeshPartition
{

#if WITH_EDITOR
FText UPCGBakeMeshTerrainSectionMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Bake Mesh Terrain Section Mesh");
}

FText UPCGBakeMeshTerrainSectionMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Renders a mesh terrain section into a texture by UV unwrap.");
}
#endif // WITH_EDITOR

FString UPCGBakeMeshTerrainSectionMeshSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMeshAttribute>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(BakeParams.Attribute)).ToString();
	}

	return FString();
}

TArray<FPCGPinProperties> UPCGBakeMeshTerrainSectionMeshSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, FPCGDataTypeInfoMeshTerrainSection::AsId(), /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false);
	InputPin.SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGBakeMeshTerrainSectionMeshSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& Pin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::RenderTarget);
	Pin.bAllowMultipleData = false;
	return PinProperties;
}

FPCGElementPtr UPCGBakeMeshTerrainSectionMeshSettings::CreateElement() const
{
	return MakeShared<FPCGBakeMeshTerrainSectionMeshElement>();
}

FPCGBakeMeshTerrainSectionMeshContext::~FPCGBakeMeshTerrainSectionMeshContext()
{
	if (bPinnedLOD0)
	{
		if (UStaticMesh* Mesh = StaticMesh.Get())
		{
			// Only restore if the flag is still set. If something else explicitly unpinned the mesh while we were using it, preserve that intent rather than re-pinning.
			if (Mesh->bForceMiplevelsToBeResident)
			{
				Mesh->bForceMiplevelsToBeResident = bPrevForceMipsResident;
			}
		}
	}
}

void FPCGBakeMeshTerrainSectionMeshContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(StaticMesh);
#if WITH_EDITOR
	Collector.AddReferencedObject(PreviewComponent);
#endif
	Collector.AddReferencedObject(RenderTarget);
	Collector.AddReferencedObject(OutputRenderTargetData);
}

bool FPCGBakeMeshTerrainSectionMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBakeMeshTerrainSectionMeshElement::ExecuteInternal);
	check(InContext);

	FPCGBakeMeshTerrainSectionMeshContext* Context = static_cast<FPCGBakeMeshTerrainSectionMeshContext*>(InContext);
	const UPCGBakeMeshTerrainSectionMeshSettings* Settings = Context->GetInputSettings<UPCGBakeMeshTerrainSectionMeshSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (Inputs.IsEmpty())
	{
		return true;
	}

	if (Inputs.Num() > 1)
	{
		PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGPinConstants::DefaultInputLabel, Context);
	}

	const UPCGMeshTerrainSectionData* SectionData = Cast<UPCGMeshTerrainSectionData>(Inputs[0].Data.Get());
	if (!SectionData)
	{
		return true;
	}

	AActor* Actor = SectionData->GetSectionActor();
	if (!Actor)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("InvalidActor", "Input section actor is no longer valid."), Context);
		return true;
	}

	auto SleepUntilNextTick = [Context]()
	{
		Context->bIsPaused = true;
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
		{
			FPCGContext::FSharedContext<FPCGContext> SharedContext(ContextHandle);
			if (FPCGContext* ContextPtr = SharedContext.Get())
			{
				ContextPtr->bIsPaused = false;
			}
		});
	};

	// Resolve the source mesh. Can be a static mesh or a preview mesh component.
	if (!Context->bSourceMeshSelected)
	{
		auto PickFirstNonNullMesh = [Context, Actor](TConstArrayView<TObjectPtr<UStaticMesh>> Meshes)
		{
			bool bMultiple = false;
			for (const TObjectPtr<UStaticMesh>& Mesh : Meshes)
			{
				if (!Mesh)
				{
					continue;
				}

				if (!Context->StaticMesh)
				{
					Context->StaticMesh = Mesh;
				}
				else
				{
					bMultiple = true;
					break;
				}
			}

			if (bMultiple)
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("MultipleMeshes", "Section '{0}' contains more than one mesh; only the first will be used (multiple meshes per section is not supported at this time)."), FText::FromString(Actor->GetName())), Context);
			}
		};

		if (ACompiledSection* Compiled = Cast<ACompiledSection>(Actor))
		{
#if WITH_EDITOR
			if (Compiled->IsPlaceholder())
			{
				UE_LOGF(LogPCGMegaMeshInterop, Verbose, "Compiled section '%ls' is still a placeholder; deferring until built.", *Actor->GetName());
				SleepUntilNextTick();
				return false;
			}
#endif // WITH_EDITOR

			PickFirstNonNullMesh(Compiled->GetStaticMeshes());
		}
#if WITH_EDITOR
		else if (APreviewSection* Preview = Cast<APreviewSection>(Actor))
		{
			PickFirstNonNullMesh(Preview->GetMeshes());

			if (!Context->StaticMesh)
			{
				UPreviewMeshComponent* PreviewComponent = Preview->GetPreviewMeshComponent();
				TSharedPtr<const FMeshData> PreviewMeshData = PreviewComponent ? PreviewComponent->GetMeshData() : nullptr;

				if (PreviewMeshData.IsValid() && PreviewMeshData->VertexCount() > 0)
				{
					Context->PreviewComponent = PreviewComponent;
				}
			}
		}
#endif

		bool bHasAnySource = Context->StaticMesh != nullptr;
#if WITH_EDITOR
		bHasAnySource |= Context->PreviewComponent != nullptr;
#endif

		if (!bHasAnySource)
		{
			UE_LOGF(LogPCGMegaMeshInterop, Verbose, "Section '%ls' has no mesh source available; nothing to bake.", *Actor->GetName());
			return true;
		}

		Context->bSourceMeshSelected = true;
	}

	// UStaticMesh path: defer until compile + RHI streaming are complete.
	if (UStaticMesh* Mesh = Context->StaticMesh.Get())
	{
#if WITH_EDITOR
		if (Mesh->IsCompiling())
		{
			UE_LOGF(LogPCGMegaMeshInterop, Verbose, "Section '%ls' static mesh '%ls' is still compiling; deferring.", *Actor->GetName(), *Mesh->GetName());
			SleepUntilNextTick();
			return false;
		}
#endif

		// Pin LOD0 resident on the streamer. Saved value is restored on the success/output-emit path below.
		if (!Context->bPinnedLOD0)
		{
			Context->bPrevForceMipsResident = Mesh->bForceMiplevelsToBeResident;
			Mesh->bForceMiplevelsToBeResident = true;
			Context->bPinnedLOD0 = true;
		}

		if (Mesh->HasPendingInitOrStreaming())
		{
			UE_LOGF(LogPCGMegaMeshInterop, Verbose, "Section '%ls' static mesh '%ls' has GPU resources still initializing or streaming; deferring.", *Actor->GetName(), *Mesh->GetName());
			SleepUntilNextTick();
			return false;
		}

		if (const FStaticMeshRenderData* RenderData = Mesh->GetRenderData())
		{
			const int32 CurrentFirstLOD = RenderData->GetCurrentFirstLODIdx(/*MinLODIdx=*/0);
			if (CurrentFirstLOD != 0)
			{
				UE_LOGF(LogPCGMegaMeshInterop, Verbose, "Section '%ls' static mesh '%ls': LOD0 not resident yet (current first LOD=%d); deferring.", *Actor->GetName(), *Mesh->GetName(), CurrentFirstLOD);
				SleepUntilNextTick();
				return false;
			}
		}
	}

	if (!Context->RenderTarget)
	{
		UTextureRenderTarget2D* RenderTarget = FPCGContext::NewObject_AnyThread<UTextureRenderTarget2D>(Context);
		RenderTarget->RenderTargetFormat = Settings->BakeParams.Format.GetValue();
		RenderTarget->ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
		RenderTarget->bAutoGenerateMips = false;
		RenderTarget->bCanCreateUAV = false;
		RenderTarget->TargetGamma = 1.0f;
		RenderTarget->InitAutoFormat(Settings->BakeParams.Resolution.X, Settings->BakeParams.Resolution.Y);
		RenderTarget->UpdateResourceImmediate(false);

		Context->RenderTarget = RenderTarget;

		Context->OutputRenderTargetData = FPCGContext::NewObject_AnyThread<UPCGRenderTargetData>(Context);
		Context->OutputRenderTargetData->Initialize(RenderTarget, FTransform::Identity, /*bInTakeOwnershipOfRenderTarget=*/true);
	}

	if (!Context->RenderTarget || !Context->OutputRenderTargetData)
	{
		return true;
	}

	if (!Context->bEnqueuedRenderCommand)
	{
		PCGUnwrapMesh::FUnwrapParams UnwrapParams;
		bool bPreviewPath = false;
		// Holds the static mesh LOD by ref-count across the GT->RT hop.
		TRefCountPtr<const FStaticMeshLODResources> LODRef;
#if WITH_EDITOR
		bPreviewPath = Context->PreviewComponent != nullptr;
		FMegaMeshCustomPreviewSceneProxy* PreviewSceneProxy = nullptr;
#endif

		UnwrapParams.UVChannelIndex = Settings->BakeParams.UVChannelIndex;
		UnwrapParams.Attribute = static_cast<PCGUnwrapMesh::EMeshAttribute>(Settings->BakeParams.Attribute);
		UnwrapParams.Resolution = Settings->BakeParams.Resolution;

#if WITH_EDITOR
		if (bPreviewPath)
		{
			TSharedPtr<const FMeshData> PreviewMeshData = Context->PreviewComponent->GetMeshData();
			if (!PreviewMeshData.IsValid())
			{
				return true;
			}

			const uint32 NumTris = static_cast<uint32>(PreviewMeshData->TriangleCount());
			const uint32 NumTexCoords = static_cast<uint32>(PreviewMeshData->GetNumUVChannels());

			if (NumTris == 0)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("PreviewZeroTris", "Section '{0}' preview mesh has zero triangles."), FText::FromString(Actor->GetName())), Context);
				return true;
			}
			else if (NumTexCoords == 0)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("PreviewNoUVs", "Section '{0}' preview mesh has no UV channels."), FText::FromString(Actor->GetName())), Context);
				return true;
			}
			else if (static_cast<uint32>(Settings->BakeParams.UVChannelIndex) >= NumTexCoords)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("PreviewUVChannelOutOfRange", "Section '{0}' preview mesh has {1} UV channel(s); requested UVChannelIndex {2} is out of range."), FText::FromString(Actor->GetName()), NumTexCoords, Settings->BakeParams.UVChannelIndex), Context);
				return true;
			}

			PreviewSceneProxy = Context->PreviewComponent->GetCustomSceneProxy();
			if (!PreviewSceneProxy)
			{
				UE_LOGF(LogPCGMegaMeshInterop, Verbose, "Section '%ls' preview mesh component has no scene proxy yet; deferring.", *Actor->GetName());
				SleepUntilNextTick();
				return false;
			}
		}
		else
#endif
		{
			UStaticMesh* Mesh = Context->StaticMesh.Get();
			check(Mesh);

			const FStaticMeshRenderData* RenderData = Mesh->GetRenderData();
			if (!RenderData || RenderData->LODResources.IsEmpty())
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NoRenderData", "Static mesh '{0}' has no render data."), FText::FromName(Mesh->GetFName())), Context);
				return true;
			}

			const FStaticMeshLODResources& LOD = RenderData->LODResources[0];
			const uint32 NumTris = LOD.GetNumTriangles();
			const uint32 NumTexCoords = LOD.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();

			if (NumTris == 0)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ZeroTris", "Static mesh '{0}' LOD0 has zero triangles (Nanite coarse fallback may be missing)."), FText::FromName(Mesh->GetFName())), Context);
				return true;
			}
			else if (NumTexCoords == 0)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NoUVs", "Static mesh '{0}' has no UV channels."), FText::FromName(Mesh->GetFName())), Context);
				return true;
			}
			else if (static_cast<uint32>(Settings->BakeParams.UVChannelIndex) >= NumTexCoords)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("UVChannelOutOfRange", "Static mesh '{0}' has {1} UV channel(s); requested UVChannelIndex {2} is out of range."), FText::FromName(Mesh->GetFName()), NumTexCoords, Settings->BakeParams.UVChannelIndex), Context);
				return true;
			}
			// Defensive check: a non-resident or stripped LOD can have NumTriangles > 0 in metadata while VertexBuffer GPU storage is 0 bytes.
			else if (LOD.GetNumVertices() == 0 || LOD.BuffersSize == 0)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("LODNotResident", "Static mesh '{0}' LOD0 buffers are not resident (NumVerts={1}, BuffersSize={2}). Likely streamed-out or stripped."), FText::FromName(Mesh->GetFName()), LOD.GetNumVertices(), LOD.BuffersSize), Context);
				return true;
			}

			// Capture the LOD by ref-count; RHI buffer refs are pulled on the render thread (see lambda below).
			LODRef = TRefCountPtr<const FStaticMeshLODResources>(&LOD);
		}

		const int32 PaddingIterations = (Settings->BakeParams.Attribute == EPCGMeshAttribute::Mask) ? 0 : Settings->BakeParams.Padding;

		Context->bIsPaused = true;

		ENQUEUE_RENDER_COMMAND(PCGBakeMeshTerrainSectionMesh)(
			[ContextHandle = Context->GetOrCreateHandle()
			, UnwrapParams
			, LODRef = MoveTemp(LODRef)
			, PaddingIterations
#if WITH_EDITOR
			, PreviewSceneProxy
#endif
			](FRHICommandListImmediate& RHICmdList) mutable
			{
				FPCGContext::FSharedContext<FPCGBakeMeshTerrainSectionMeshContext> SharedContext(ContextHandle);
				FPCGBakeMeshTerrainSectionMeshContext* LocalContext = SharedContext.Get();
				if (!LocalContext)
				{
					return;
				}

				ON_SCOPE_EXIT
				{
					LocalContext->bIsPaused = false;
				};

#if WITH_EDITOR
				if (PreviewSceneProxy)
				{
					UnwrapParams.PositionBufferRHI = PreviewSceneProxy->GetPositionVertexBufferRHI();
					UnwrapParams.TexCoordBufferRHI = PreviewSceneProxy->GetTexCoordVertexBufferRHI();
					UnwrapParams.IndexBufferRHI = PreviewSceneProxy->GetIndexBufferRHI();
					UnwrapParams.bFullPrecisionUVs = PreviewSceneProxy->GetUseFullPrecisionUVs();
					UnwrapParams.NumVerts = PreviewSceneProxy->GetNumVertices();
					UnwrapParams.NumTris = PreviewSceneProxy->GetNumTriangles();
					UnwrapParams.NumTexCoords = PreviewSceneProxy->GetNumTexCoords();
				}
				else
#endif
				{
					check(LODRef);
					UnwrapParams.InitFromLOD(*LODRef);
				}

				if (!PCGUnwrapMesh::ValidateParams(UnwrapParams))
				{
					return;
				}

				UTextureRenderTarget2D* RenderTarget = LocalContext->RenderTarget;
				FTextureRenderTargetResource* RTResource = RenderTarget ? RenderTarget->GetRenderTargetResource() : nullptr;
				FTextureRHIRef TargetRHI = RTResource ? RTResource->GetRenderTargetTexture() : FTextureRHIRef();
				if (!TargetRHI)
				{
					return;
				}

				FRDGBuilder GraphBuilder(RHICmdList);
				FRDGTextureRef RDGOutput = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(TargetRHI, TEXT("PCGBakeMeshTerrainSectionMesh.Output")));
				if (PCGUnwrapMesh::AddUnwrapMeshPass(GraphBuilder, RDGOutput, UnwrapParams))
				{
					PCGDilate::AddDilatePass(GraphBuilder, RDGOutput, PaddingIterations);
				}
				else
				{
					PCGLog::LogErrorOnGraph(LOCTEXT("UnwrapPassFailed", "Failed to add unwrap mesh pass."), LocalContext);
				}
				GraphBuilder.Execute();
			});

		Context->bEnqueuedRenderCommand = true;
		return false;
	}

	FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
	Output.Data = Context->OutputRenderTargetData;
	Output.Pin = PCGPinConstants::DefaultOutputLabel;

	return true;
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
