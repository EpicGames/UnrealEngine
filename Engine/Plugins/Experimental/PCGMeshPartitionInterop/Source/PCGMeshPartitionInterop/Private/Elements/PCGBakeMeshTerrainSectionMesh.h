// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"
#include "Elements/PCGMeshAttribute.h"

#include "Engine/TextureRenderTarget2D.h"

#include "PCGBakeMeshTerrainSectionMesh.generated.h"

class UPCGRenderTargetData;
class UStaticMesh;
class UTextureRenderTarget2D;

namespace UE::MeshPartition
{
class UPreviewMeshComponent;

/** Renders a mesh terrain section into a texture by UV unwrap. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGBakeMeshTerrainSectionMeshSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("BakeMeshTerrainSectionMesh")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGBakeMeshAttributesParams BakeParams;
};

struct FPCGBakeMeshTerrainSectionMeshContext : public FPCGContext
{
public:
	virtual ~FPCGBakeMeshTerrainSectionMeshContext();

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

public:
	/** Static mesh of the section actor that we are rendering. */
	TObjectPtr<UStaticMesh> StaticMesh;

#if WITH_EDITOR
	/** Set instead of StaticMesh when we fall back to the preview proxy path. */
	TObjectPtr<UPreviewMeshComponent> PreviewComponent;
#endif

	/** Render target the pass writes into. */
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

	/** PCG-typed view of the render target produced as the node's output. */
	TObjectPtr<UPCGRenderTargetData> OutputRenderTargetData;

	bool bSourceMeshSelected = false;
	bool bEnqueuedRenderCommand = false;

	/** True once we have asked the streamer to force LOD0 resident for this static mesh. */
	bool bPinnedLOD0 = false;

	// Snapshot of bForceMiplevelsToBeResident captured before we set it. Restored when execution exits.
	bool bPrevForceMipsResident = false;
};

class FPCGBakeMeshTerrainSectionMeshElement : public IPCGElementWithCustomContext<FPCGBakeMeshTerrainSectionMeshContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

} // namespace UE::MeshPartition
