// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"

#include "PCGGetMeshPartitionGrassTypes.generated.h"

namespace UE::MeshPartition
{

class UMeshPartitionDefinition;


/** Emits an attribute set with the landscape grass types declared by a mesh partition definition's material. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGGetMeshPartitionGrassTypesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetMeshPartitionGrassTypes")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~ End UPCGSettings interface

public:
	/** Mesh partition definition whose material is queried for landscape grass types. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<UMeshPartitionDefinition> Definition;

	/** Output attribute name for the material-side grass name (the FName key on the LandscapeGrassOutput input). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName GrassNameAttributeName = FName(TEXT("GrassName"));

	/** Output attribute name for the soft object path to the ULandscapeGrassType asset. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName GrassTypeAttributeName = FName(TEXT("GrassType"));

	/** By default, the definition is loaded asynchronously; toggle to force synchronous load. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGGetMeshPartitionGrassTypesContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGGetMeshPartitionGrassTypesElement : public IPCGElementWithCustomContext<FPCGGetMeshPartitionGrassTypesContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

} // namespace UE::MeshPartition
