// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetMeshPartitionTexelSizes.h"

#include "PCGParamData.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Utils/PCGLogErrors.h"

#include "MeshPartitionDefinition.h"

#define LOCTEXT_NAMESPACE "PCGGetMeshPartitionTexelSizesElement"

namespace UE::MeshPartition
{

#if WITH_EDITOR
FText UPCGGetMeshPartitionTexelSizesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Mesh Partition Texel Sizes");
}

FText UPCGGetMeshPartitionTexelSizesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Emits an attribute set with the channel textures and material cache texel sizes from a mesh partition definition.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGetMeshPartitionTexelSizesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& Pin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);
	Pin.bAllowMultipleData = false;
	return PinProperties;
}

FPCGElementPtr UPCGGetMeshPartitionTexelSizesSettings::CreateElement() const
{
	return MakeShared<FPCGGetMeshPartitionTexelSizesElement>();
}

bool FPCGGetMeshPartitionTexelSizesElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetMeshPartitionTexelSizesElement::PrepareDataInternal);
	check(InContext);

	const UPCGGetMeshPartitionTexelSizesSettings* Settings = InContext->GetInputSettings<UPCGGetMeshPartitionTexelSizesSettings>();
	check(Settings);

	if (Settings->Definition.IsNull())
	{
		return true;
	}

	FPCGGetMeshPartitionTexelSizesContext* Context = static_cast<FPCGGetMeshPartitionTexelSizesContext*>(InContext);

	if (!Context->WasLoadRequested())
	{
		return Context->RequestResourceLoad(Context, { Settings->Definition.ToSoftObjectPath() }, !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGGetMeshPartitionTexelSizesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetMeshPartitionTexelSizesElement::ExecuteInternal);
	check(InContext);

	const UPCGGetMeshPartitionTexelSizesSettings* Settings = InContext->GetInputSettings<UPCGGetMeshPartitionTexelSizesSettings>();
	check(Settings);

	const UMeshPartitionDefinition* Definition = Settings->Definition.Get();
	if (!Definition)
	{
		return true;
	}

	UPCGParamData* OutParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
	check(OutParamData && OutParamData->Metadata);

	FPCGMetadataAttribute<float>* ChannelAttribute = OutParamData->Metadata->CreateAttribute<float>(Settings->ChannelTexturesTexelSizeAttributeName, 0.0f, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
	if (!ChannelAttribute)
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ChannelAttributeCreateFailed", "Failed to create attribute '{0}' on output parameter data."), FText::FromName(Settings->ChannelTexturesTexelSizeAttributeName)), InContext);
		return true;
	}

	FPCGMetadataAttribute<float>* MaterialCacheAttribute = OutParamData->Metadata->CreateAttribute<float>(Settings->MaterialCacheTexelSizeAttributeName, 0.0f, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
	if (!MaterialCacheAttribute)
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MaterialCacheAttributeCreateFailed", "Failed to create attribute '{0}' on output parameter data."), FText::FromName(Settings->MaterialCacheTexelSizeAttributeName)), InContext);
		return true;
	}

	const PCGMetadataEntryKey EntryKey = OutParamData->Metadata->AddEntry();
	ChannelAttribute->SetValue(EntryKey, Definition->GetChannelTexelSize());
	MaterialCacheAttribute->SetValue(EntryKey, Definition->GetMaterialCacheTexelSize());

	FPCGTaggedData& Output = InContext->OutputData.TaggedData.Emplace_GetRef();
	Output.Data = OutParamData;
	Output.Pin = PCGPinConstants::DefaultOutputLabel;

	return true;
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
