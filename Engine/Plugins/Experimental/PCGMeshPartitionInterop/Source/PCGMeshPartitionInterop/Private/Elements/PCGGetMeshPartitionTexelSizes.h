// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"

#include "PCGGetMeshPartitionTexelSizes.generated.h"

namespace UE::MeshPartition
{

class UMeshPartitionDefinition;


/** Emits an attribute set with the channel textures and material cache texel sizes from a mesh partition definition. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGGetMeshPartitionTexelSizesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetMeshPartitionTexelSizes")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~ End UPCGSettings interface

public:
	/** Mesh partition definition to read texel sizes from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<UMeshPartitionDefinition> Definition;

	/** Output attribute name for the channel textures texel size (in world units). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName ChannelTexturesTexelSizeAttributeName = FName(TEXT("ChannelTexturesTexelSize"));

	/** Output attribute name for the material cache texel size (in world units). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName MaterialCacheTexelSizeAttributeName = FName(TEXT("MaterialCacheTexelSize"));

	/** By default, the definition is loaded asynchronously; toggle to force synchronous load. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGGetMeshPartitionTexelSizesContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGGetMeshPartitionTexelSizesElement : public IPCGElementWithCustomContext<FPCGGetMeshPartitionTexelSizesContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

} // namespace UE::MeshPartition
