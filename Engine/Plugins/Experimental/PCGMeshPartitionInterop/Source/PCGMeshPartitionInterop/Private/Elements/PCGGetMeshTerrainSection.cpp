// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetMeshTerrainSection.h"

#include "Data/PCGMeshTerrainSectionData.h"

#include "PCGGraphExecutionStateInterface.h"
#include "PCGModule.h"
#include "Elements/PCGActorSelector.h"
#include "Helpers/PCGHelpers.h"

#include "Engine/World.h"
#include "GameFramework/Actor.h"

#include "MeshPartition.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionStaticMeshComponent.h"

#if WITH_EDITOR
#include "PCGMeshPartitionInteropModule.h"
#include "Data/PCGMeshPartitionSelectionKey.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionPreviewSection.h"
#endif

#define LOCTEXT_NAMESPACE "PCGGetMeshTerrainSectionElement"

namespace UE::MeshPartition
{

#if WITH_EDITOR
FText UPCGGetMeshTerrainSectionSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Mesh Terrain Section");
}

FText UPCGGetMeshTerrainSectionSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Emits one Mesh Terrain Section data per mesh terrain section that overlaps the generation volume.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGetMeshTerrainSectionSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoMeshTerrainSection::AsId());
	return PinProperties;
}

FPCGElementPtr UPCGGetMeshTerrainSectionSettings::CreateElement() const
{
	return MakeShared<FPCGGetMeshTerrainSectionElement>();
}

bool FPCGGetMeshTerrainSectionElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetMeshTerrainSectionElement::ExecuteInternal);

	const UPCGGetMeshTerrainSectionSettings* Settings = InContext->GetInputSettings<UPCGGetMeshTerrainSectionSettings>();
	check(Settings);

	IPCGGraphExecutionSource* ExecutionSource = InContext->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		return true;
	}

	const FBox InputBounds = PCGHelpers::GetGridBounds(ExecutionSource->GetExecutionState().GetTypedTarget<AActor>(), ExecutionSource);
	if (!InputBounds.IsValid)
	{
		return true;
	}

	auto EmitSectionActor = [InContext](AActor* SectionActor)
	{
		UPCGMeshTerrainSectionData* OutData = FPCGContext::NewObject_AnyThread<UPCGMeshTerrainSectionData>(InContext);
		OutData->SetSectionActor(SectionActor);

		FPCGTaggedData& Output = InContext->OutputData.TaggedData.Emplace_GetRef();
		Output.Data = OutData;
	};

	FPCGActorSelectorSettings ActorSelector;
	ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;
	ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
	ActorSelector.bSelectMultiple = true;

	auto SelfIgnoreCheck = [](const AActor*) -> bool { return true; };

	UWorld* World = ExecutionSource->GetExecutionState().GetWorld();

	if (World && World->IsGameWorld())
	{
		// Runtime path: enumerate ACompiledSection actors directly.
		ActorSelector.ActorSelectionClass = ACompiledSection::StaticClass();

		// Only mesh components contribute to scattering.
		auto GetMeshComponentBounds = [](const ACompiledSection* CompiledSection) -> FBox
		{
			FBox Bounds(ForceInit);

			if (CompiledSection)
			{
				for (const UMeshPartitionStaticMeshComponent* MeshComponent : CompiledSection->GetMeshComponents())
				{
					Bounds += MeshComponent->Bounds.GetBox();
				}
			}

			return Bounds;
		};

		auto BoundsCheck = [&InputBounds, &GetMeshComponentBounds](const AActor* Actor) -> bool
		{
			const FBox Bounds = GetMeshComponentBounds(Cast<ACompiledSection>(Actor));
			return Bounds.IsValid && InputBounds.Overlap(Bounds).GetVolume() > UE_KINDA_SMALL_NUMBER;
		};

		TArray<AActor*> CompiledSectionActors = PCGActorSelector::FindActors(ActorSelector, ExecutionSource, BoundsCheck, SelfIgnoreCheck);
		for (AActor* Actor : CompiledSectionActors)
		{
			if (ACompiledSection* CompiledSection = Cast<ACompiledSection>(Actor))
			{
				EmitSectionActor(CompiledSection);
			}
		}
	}
#if WITH_EDITOR
	else
	{
		// Editor path: preview sections are owned by the partition's editor component, not top-level world actors.
		// The mesh partition actor's own bounds are not representative of its preview sections' bounds, so bounds filtering is deferred until we iterate preview sections below.
		ActorSelector.ActorSelectionClass = AMeshPartition::StaticClass();

		auto BoundsCheck = [](const AActor*) -> bool { return true; };

		TArray<AActor*> MeshPartitionActors = PCGActorSelector::FindActors(ActorSelector, ExecutionSource, BoundsCheck, SelfIgnoreCheck);

		// While a partition's preview sections are rebuilding, the existing actors are still valid but queued for destruction once the new build completes.
		// Emitting them now would feed downstream consumers actors that go null mid-execution. Check again next tick.
		for (AActor* PartitionActor : MeshPartitionActors)
		{
			AMeshPartition* MeshPartition = Cast<AMeshPartition>(PartitionActor);
			if (!MeshPartition)
			{
				continue;
			}

			UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
			if (EditorComponent && EditorComponent->IsAnyPreviewSectionBuildActive())
			{
				UE_LOGF(LogPCGMegaMeshInterop, Verbose, "Mesh partition '%ls' has an active preview section build; deferring.", *MeshPartition->GetName());

				InContext->bIsPaused = true;

				FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = InContext->GetOrCreateHandle()]()
				{
					FPCGContext::FSharedContext<FPCGContext> SharedContext(ContextHandle);
					if (FPCGContext* ContextPtr = SharedContext.Get())
					{
						ContextPtr->bIsPaused = false;
					}
				});

				return false;
			}
		}

		FPCGDynamicTrackingHelper DynamicTracking;
		DynamicTracking.EnableAndInitialize(InContext, /*NumKeysPerActor=*/2 * MeshPartitionActors.Num());

		for (AActor* Actor : MeshPartitionActors)
		{
			AMeshPartition* MeshPartition = Cast<AMeshPartition>(Actor);
			if (!MeshPartition)
			{
				continue;
			}

			UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
			if (!EditorComponent)
			{
				continue;
			}

			// Modifier changes emit FPCGLayerSelectionKey(Final); bounds-only changes emit FPCGGlobalSelectionKey.
			// Register both so per-modifier and global notifications both reach this node.
			FPCGSelectionKey GlobalKey = FPCGSelectionKey::CreateFromObjectPtr(MeshPartition);
			FPCGSelectionKey FinalKey(GlobalKey);

			FinalKey.CustomKey.InitializeAs<FPCGLayerSelectionKey>(EPCGQueryType::Final, /*LayerName=*/ NAME_None, /*SubPriority=*/0, /*bIsFromPrevious=*/false, EPCGLayerSelectionKeyType::Listener);
			DynamicTracking.AddToTracking(MoveTemp(FinalKey), /*bIsCulled=*/true);

			GlobalKey.CustomKey.InitializeAs<FPCGGlobalSelectionKey>();
			DynamicTracking.AddToTracking(MoveTemp(GlobalKey), /*bIsCulled=*/true);

			EditorComponent->ForAllPreviewSections([&EmitSectionActor, &InputBounds](APreviewSection* PreviewSection)
			{
				if (!PreviewSection)
				{
					return true;
				}

				const FBox PreviewBounds = PreviewSection->GetPreviewMeshBounds();
				if (!PreviewBounds.IsValid || InputBounds.Overlap(PreviewBounds).GetVolume() <= UE_KINDA_SMALL_NUMBER)
				{
					return true;
				}

				EmitSectionActor(PreviewSection);
				return true;
			});
		}

		DynamicTracking.Finalize(InContext);
	}
#endif // WITH_EDITOR

	return true;
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
