// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"

#include "PCGGetMeshTerrainSection.generated.h"

namespace UE::MeshPartition
{

/** Emits one UPCGMeshTerrainSectionData per mesh terrain section that overlaps the generation volume. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGGetMeshTerrainSectionSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetMeshTerrainSection")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return {}; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~ End UPCGSettings interface
};

class FPCGGetMeshTerrainSectionElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

} // namespace UE::MeshPartition
