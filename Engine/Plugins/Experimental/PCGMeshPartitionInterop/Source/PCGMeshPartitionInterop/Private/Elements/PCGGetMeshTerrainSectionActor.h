// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"

#include "PCGGetMeshTerrainSectionActor.generated.h"

namespace UE::MeshPartition
{

/** Emits an attribute set with the soft object path to the actor of the given mesh terrain section. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGGetMeshTerrainSectionActorSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetMeshTerrainSectionActor")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~ End UPCGSettings interface

public:
	/** Output attribute name for the section actor soft reference. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName OutputAttributeName = FName(TEXT("ActorReference"));
};

class FPCGGetMeshTerrainSectionActorElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

} // namespace UE::MeshPartition
