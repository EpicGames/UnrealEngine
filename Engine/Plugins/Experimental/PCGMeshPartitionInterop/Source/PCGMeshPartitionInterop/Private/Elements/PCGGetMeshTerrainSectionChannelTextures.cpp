// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetMeshTerrainSectionChannelTextures.h"

#include "Compute/PCGComputeCommon.h"
#include "Data/PCGMeshTerrainSectionData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGTextureData.h"
#include "Utils/PCGLogErrors.h"

#include "PCGModule.h"

#include "MeshPartition.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionDefinition.h"

#include "Algo/AnyOf.h"
#include "Containers/Ticker.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "GameFramework/Actor.h"
#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIResources.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "MeshPartitionPreviewSection.h"
#endif

#define LOCTEXT_NAMESPACE "PCGGetMeshTerrainSectionChannelTexturesElement"

namespace UE::MeshPartition
{

namespace PCGGetMeshTerrainSectionChannelTexturesHelpers
{
	// Sentinel used in ChannelTable for "channel not present in this section".
	// Mirrors the "-1" written by MeshPartition (see MeshPartitionChannelCollection.cpp); cast to uint8 = 0xFF.
	constexpr uint8 AbsentChannelSentinel = 0xFF;

	const UMeshPartitionDefinition* GetMeshPartitionDefinition(AActor* SectionActor)
	{
		if (ACompiledSection* Compiled = Cast<ACompiledSection>(SectionActor))
		{
			if (AMeshPartition* Parent = Compiled->GetParentMegaMesh())
			{
				return Parent->GetMeshPartitionDefinition();
			}
		}
#if WITH_EDITOR
		if (APreviewSection* Preview = Cast<APreviewSection>(SectionActor))
		{
			if (AMeshPartition* Parent = Preview->GetParent())
			{
				return Parent->GetMeshPartitionDefinition();
			}
		}
#endif
		return nullptr;
	}

	const TArray<uint8>* GetChannelTable(AActor* SectionActor)
	{
		if (ACompiledSection* Compiled = Cast<ACompiledSection>(SectionActor))
		{
			return &Compiled->GetChannelTable();
		}
#if WITH_EDITOR
		if (APreviewSection* Preview = Cast<APreviewSection>(SectionActor))
		{
			return &Preview->GetChannelTable();
		}
#endif
		return nullptr;
	}

	UTexture* GetChannelTexture(AActor* SectionActor)
	{
		if (ACompiledSection* Compiled = Cast<ACompiledSection>(SectionActor))
		{
			return Compiled->GetChannelTexture();
		}
#if WITH_EDITOR
		if (APreviewSection* Preview = Cast<APreviewSection>(SectionActor))
		{
			return Preview->GetChannelTexture();
		}
#endif
		return nullptr;
	}

	int32 ResolveSliceForChannel(FName ChannelName, const TArray<FName>& GlobalChannels, const TArray<uint8>& ChannelTable)
	{
		const int32 GlobalIndex = GlobalChannels.IndexOfByKey(ChannelName);
		if (GlobalIndex == INDEX_NONE || !ChannelTable.IsValidIndex(GlobalIndex))
		{
			return INDEX_NONE;
		}

		const uint8 Slice = ChannelTable[GlobalIndex];
		return Slice == AbsentChannelSentinel ? INDEX_NONE : static_cast<int32>(Slice);
	}

	bool IsTextureFullyStreamedIn(UTexture* InTexture)
	{
		return InTexture
#if WITH_EDITOR
			&& !InTexture->IsDefaultTexture()
#endif
			&& !InTexture->HasPendingInitOrStreaming()
			&& InTexture->IsFullyStreamedIn();
	}
}

#if WITH_EDITOR
FText UPCGGetMeshTerrainSectionChannelTexturesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Mesh Terrain Section Channel Textures");
}

FText UPCGGetMeshTerrainSectionChannelTexturesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Emits the input mesh terrain section's baked channel textures.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGetMeshTerrainSectionChannelTexturesSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, FPCGDataTypeInfoMeshTerrainSection::AsId(), /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false);
	InputPin.SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetMeshTerrainSectionChannelTexturesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	if (bOutputTextureArray)
	{
		FPCGPinProperties& Pin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoTexture2DArray::AsId());
		Pin.bAllowMultipleData = false;
	}
	else
	{
		PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoTexture2D::AsId());
	}
	return PinProperties;
}

FPCGElementPtr UPCGGetMeshTerrainSectionChannelTexturesSettings::CreateElement() const
{
	return MakeShared<FPCGGetMeshTerrainSectionChannelTexturesElement>();
}

FPCGGetMeshTerrainSectionChannelTexturesContext::~FPCGGetMeshTerrainSectionChannelTexturesContext()
{
	if (bStreamingRequested)
	{
		if (UTexture* SourceTexture = SourceChannelTextureArray.Get())
		{
			// Only restore if the flag is still set. If something else explicitly unpinned the texture while we were using it, preserve that intent rather than re-pinning.
			if (SourceTexture->bForceMiplevelsToBeResident)
			{
				SourceTexture->bForceMiplevelsToBeResident = bPriorForceMipsResident;
			}
		}
	}
}

void FPCGGetMeshTerrainSectionChannelTexturesContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	if (OutputTextureArrayData)
	{
		Collector.AddReferencedObject(OutputTextureArrayData);
	}

	Collector.AddReferencedObjects(OutputTextureDatas);

	if (BlackTexture)
	{
		Collector.AddReferencedObject(BlackTexture);
	}
}

bool FPCGGetMeshTerrainSectionChannelTexturesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetMeshTerrainSectionChannelTexturesElement::ExecuteInternal);

	FPCGGetMeshTerrainSectionChannelTexturesContext* Context = static_cast<FPCGGetMeshTerrainSectionChannelTexturesContext*>(InContext);
	const UPCGGetMeshTerrainSectionChannelTexturesSettings* Settings = Context->GetInputSettings<UPCGGetMeshTerrainSectionChannelTexturesSettings>();
	check(Settings);

	// Phase 1: Resolve channels (one-shot).
	if (!Context->bChannelsResolved)
	{
		const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		if (Inputs.IsEmpty())
		{
			return true;
		}

		if (Inputs.Num() > 1)
		{
			PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGPinConstants::DefaultInputLabel, Context);
		}

		const UPCGMeshTerrainSectionData* SectionData = Cast<UPCGMeshTerrainSectionData>(Inputs[0].Data.Get());
		if (!SectionData)
		{
			return true;
		}

		AActor* Actor = SectionData->GetSectionActor();
		if (!Actor)
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("InvalidActor", "Input section actor is not valid."), Context);
			return true;
		}

		UTexture* ChannelTextureArray = PCGGetMeshTerrainSectionChannelTexturesHelpers::GetChannelTexture(Actor);
		if (!ChannelTextureArray)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("NoChannelTexture", "Section '{0}' has no channel texture."), FText::FromString(Actor->GetName())), Context);
			return true;
		}

		const TArray<uint8>* ChannelTablePtr = PCGGetMeshTerrainSectionChannelTexturesHelpers::GetChannelTable(Actor);
		if (!ChannelTablePtr)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("NoChannelTable", "Section '{0}' has no channel table."), FText::FromString(Actor->GetName())), Context);
			return true;
		}

		const UMeshPartitionDefinition* Definition = PCGGetMeshTerrainSectionChannelTexturesHelpers::GetMeshPartitionDefinition(Actor);
		if (!Definition)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("NoDefinition", "Section '{0}' has no mesh partition definition."), FText::FromString(Actor->GetName())), Context);
			return true;
		}

		const TArray<FName> GlobalChannels = Definition->GetChannelNames();

		// Build the effective channel list and slice mapping. Channels missing from this section resolve to INDEX_NONE; whether
		// they are dropped from the output or retained as zero/black slots is controlled by bDropMissingChannels.
		if (Settings->bExcludeSelectedChannels)
		{
			// Exclusion: every global channel except those selected, in global order.
			Context->EffectiveChannelNames.Reserve(GlobalChannels.Num());
			Context->EffectiveSliceIndices.Reserve(GlobalChannels.Num());

			for (const FName& Name : GlobalChannels)
			{
				if (Settings->SelectedChannels.Contains(Name))
				{
					continue;
				}

				const int32 Slice = PCGGetMeshTerrainSectionChannelTexturesHelpers::ResolveSliceForChannel(Name, GlobalChannels, *ChannelTablePtr);

				if (Slice == INDEX_NONE && Settings->bDropMissingChannels)
				{
					continue;
				}

				Context->EffectiveChannelNames.Add(Name);
				Context->EffectiveSliceIndices.Add(Slice);
			}
		}
		else
		{
			// Inclusion: exactly SelectedChannels in user order.
			Context->EffectiveChannelNames.Reserve(Settings->SelectedChannels.Num());
			Context->EffectiveSliceIndices.Reserve(Settings->SelectedChannels.Num());

			for (const FName& Name : Settings->SelectedChannels)
			{
				const int32 Slice = PCGGetMeshTerrainSectionChannelTexturesHelpers::ResolveSliceForChannel(Name, GlobalChannels, *ChannelTablePtr);

				if (Slice == INDEX_NONE && Settings->bDropMissingChannels)
				{
					continue;
				}

				Context->EffectiveChannelNames.Add(Name);
				Context->EffectiveSliceIndices.Add(Slice);
			}
		}

		if (Context->EffectiveChannelNames.IsEmpty())
		{
			return true;
		}

		Context->SourceChannelTextureArray = ChannelTextureArray;
		Context->SectionActor = Actor;

		// Fast path: skip the GPU composite when the source array's slices already match the output ordering 1:1.
		if (Settings->bOutputTextureArray)
		{
			bool bIsIdentityMapping = true;
			for (int32 EffectiveSliceIndex = 0; EffectiveSliceIndex < Context->EffectiveSliceIndices.Num(); ++EffectiveSliceIndex)
			{
				if (Context->EffectiveSliceIndices[EffectiveSliceIndex] != EffectiveSliceIndex)
				{
					bIsIdentityMapping = false;
					break;
				}
			}

			if (bIsIdentityMapping)
			{
				if (UTexture2DArray* SourceArray = Cast<UTexture2DArray>(ChannelTextureArray))
				{
					if (SourceArray->GetArraySize() == Context->EffectiveSliceIndices.Num())
					{
						Context->bUseFastPath = true;
					}
				}
			}
		}

		Context->bChannelsResolved = true;
	}

	UTexture* SourceTexture = Context->SourceChannelTextureArray.Get();
	AActor* SectionActor = Context->SectionActor.Get();
	if (!SourceTexture || !SectionActor)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("StaleSection", "Section actor or channel texture became invalid during execution."), Context);
		return true;
	}

	// Phase 2: Wait for the source channel texture to be fully streamed in.
	if (!Context->bStreamingComplete)
	{
		if (!Context->bStreamingRequested)
		{
			// Snapshot bForceMiplevelsToBeResident before flipping it so we can restore the setting when execution finishes.
			// @todo_pcg: Fast path and per-channel mode both wrap the source UTexture in the output data. Once this element exits the mip residency pin is released and the streamer can evict mips out from under downstream samplers.
			Context->bPriorForceMipsResident = SourceTexture->bForceMiplevelsToBeResident;
			SourceTexture->bForceMiplevelsToBeResident = true;
			Context->bStreamingRequested = true;
		}

		if (!PCGGetMeshTerrainSectionChannelTexturesHelpers::IsTextureFullyStreamedIn(SourceTexture))
		{
			if (Context->StreamingWaitStartTime == 0.0)
			{
				Context->StreamingWaitStartTime = FPlatformTime::Seconds();
				UE_LOGF(LogPCG, Log, "Waiting for channel texture '%ls' on section '%ls' to stream in.", *SourceTexture->GetName(), *SectionActor->GetName());
			}

			Context->bIsPaused = true;

			FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
			{
				FPCGContext::FSharedContext<FPCGGetMeshTerrainSectionChannelTexturesContext> SharedContext(ContextHandle);
				if (FPCGGetMeshTerrainSectionChannelTexturesContext* ContextPtr = SharedContext.Get())
				{
					ContextPtr->bIsPaused = false;
				}
			});

			return false;
		}

		if (Context->StreamingWaitStartTime > 0.0)
		{
			const double WaitSeconds = FPlatformTime::Seconds() - Context->StreamingWaitStartTime;
			UE_LOGF(LogPCG, Log, "Channel texture '%ls' on section '%ls' streamed in after %.2fs.", *SourceTexture->GetName(), *SectionActor->GetName(), WaitSeconds);
		}

		Context->bStreamingComplete = true;
	}

	// Phase 3: GPU composite. Only needed in array mode without the fast path.
	const bool bNeedsComposite = Settings->bOutputTextureArray && !Context->bUseFastPath;
	if (bNeedsComposite && !Context->bRenderingComplete)
	{
		if (!Context->bCompositeScheduled)
		{
			FTextureResource* Resource = SourceTexture->GetResource();
			FRHITexture* SourceRHI = nullptr;

			if (Resource)
			{
				SourceRHI = Resource->GetTextureRHI();
			}

			if (!SourceRHI)
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("NoSourceRHI", "Section '{0}' channel texture has no RHI resource."), FText::FromString(SectionActor->GetName())), Context);
				return true;
			}

			Context->bIsPaused = true;
			Context->bCompositeScheduled = true;

			ENQUEUE_RENDER_COMMAND(PCGCompositeMeshTerrainChannelTextures)(
				[WeakHandle = Context->GetOrCreateHandle(), SourceRHIRef = FTextureRHIRef(SourceRHI)](FRHICommandListImmediate& RHICmdList)
				{
					FPCGContext::FSharedContext<FPCGGetMeshTerrainSectionChannelTexturesContext> SharedContext(WeakHandle);
					FPCGGetMeshTerrainSectionChannelTexturesContext* ContextPtr = SharedContext.Get();
					if (!ContextPtr)
					{
						return;
					}

					const TArray<int32>& SliceMapping = ContextPtr->EffectiveSliceIndices;
					const FRHITextureDesc& SourceDesc = SourceRHIRef->GetDesc();
					const FIntPoint Resolution(SourceDesc.Extent.X, SourceDesc.Extent.Y);

					FRDGBuilder GraphBuilder(RHICmdList);

					FRDGTextureRef SrcRDG = RegisterExternalTexture(GraphBuilder, SourceRHIRef.GetReference(), TEXT("PCGSrcChannelArray"));

					// Narrow the source format to a render-targetable equivalent. BC and other UAV/RT-incompatible formats
					// (e.g. BC4 weight textures) cannot be the dest of an RT clear or be written as typed UAV, so we route
					// them through PCG's canonical render-target format and back. AddDrawTexturePass still takes the
					// hardware-copy fast path when source and dest formats match (i.e. non-BC sources pass through unchanged).
					const EPCGRenderTargetFormat ComputeFormat = PCGComputeHelpers::GetPCGRenderTargetFormatFromPixelFormat(SourceDesc.Format);
					const EPixelFormat DstFormat = PCGComputeHelpers::GetPixelFormatFromPCGRenderTargetFormat(ComputeFormat);

					// TargetArraySlicesIndependently is required so we can bind individual non-zero slices as RT / per-slice
					// clear; otherwise the RHI rejects FirstSliceIndex > 0 unless GRHIGlobals.SupportsBindingTexArrayPerSlice.
					const FRDGTextureDesc DstDesc = FRDGTextureDesc::Create2DArray(Resolution, DstFormat, FClearValueBinding::Transparent, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::TargetArraySlicesIndependently, static_cast<uint16>(SliceMapping.Num()), /*NumMips=*/1);
					FRDGTextureRef DstRDG = GraphBuilder.CreateTexture(DstDesc, TEXT("PCGCompositedChannelArray"));

					const FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

					for (int32 DstSlice = 0; DstSlice < SliceMapping.Num(); ++DstSlice)
					{
						const int32 SrcSlice = SliceMapping[DstSlice];

						if (SrcSlice >= 0)
						{
							FRDGDrawTextureInfo DrawInfo;
							DrawInfo.Size = Resolution;
							DrawInfo.SourceSliceIndex = static_cast<uint32>(SrcSlice);
							DrawInfo.DestSliceIndex = static_cast<uint32>(DstSlice);
							DrawInfo.NumSlices = 1;
							AddDrawTexturePass(GraphBuilder, ShaderMap, SrcRDG, DstRDG, DrawInfo);
						}
						else
						{
							FRDGTextureClearInfo ClearInfo;
							ClearInfo.ClearColor = FLinearColor::Transparent;
							ClearInfo.FirstSliceIndex = static_cast<uint32>(DstSlice);
							ClearInfo.NumSlices = 1;
							AddClearRenderTargetPass(GraphBuilder, DstRDG, ClearInfo);
						}
					}

					TRefCountPtr<IPooledRenderTarget> ExtractedHandle;
					GraphBuilder.QueueTextureExtraction(DstRDG, &ExtractedHandle, ERHIAccess::SRVCompute);
					GraphBuilder.Execute();

					ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakHandle, ExtractedHandle]() mutable
					{
						FPCGContext::FSharedContext<FPCGGetMeshTerrainSectionChannelTexturesContext> WakeContext(WeakHandle);
						if (FPCGGetMeshTerrainSectionChannelTexturesContext* ContextPtr = WakeContext.Get())
						{
							ContextPtr->CompositedHandle = MoveTemp(ExtractedHandle);
							ContextPtr->bRenderingComplete = true;
							ContextPtr->bIsPaused = false;
						}
					});
				});

			return false;
		}

		// Scheduled but not yet woken; remain paused.
		return false;
	}

	// Phase 4: Initialize output PCG data, polling on async resource readiness.
	bool bAllInit = true;
	const FTransform SectionTransform = SectionActor->GetActorTransform();

	if (Settings->bOutputTextureArray)
	{
		if (!Context->OutputTextureArrayData)
		{
			Context->OutputTextureArrayData = FPCGContext::NewObject_AnyThread<UPCGTexture2DArrayData>(Context);
		}

		FPCGTexture2DArrayDataInitParams Params;
		Params.Transform = SectionTransform;
		Params.Filter = Settings->Filter;

		if (Context->bUseFastPath)
		{
			bAllInit &= Context->OutputTextureArrayData->Initialize(SourceTexture, Params);
		}
		else if (Context->CompositedHandle)
		{
			bAllInit &= Context->OutputTextureArrayData->Initialize(Context->CompositedHandle, Params);
		}
		else
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("CompositeFailed", "Section '{0}' channel texture composite produced no extracted handle."), FText::FromString(SectionActor->GetName())), Context);
			return true;
		}
	}
	else
	{
		// Per-channel mode. Need a black texture for absent channels.
		const bool bHasAbsentChannel = Algo::AnyOf(Context->EffectiveSliceIndices, [](int32 Slice){ return Slice == INDEX_NONE; });
		if (bHasAbsentChannel && !Context->BlackTexture)
		{
			Context->BlackTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineResources/Black.Black"));
		}

		if (Context->OutputTextureDatas.IsEmpty())
		{
			Context->OutputTextureDatas.Reserve(Context->EffectiveChannelNames.Num());
			for (int32 Index = 0; Index < Context->EffectiveChannelNames.Num(); ++Index)
			{
				Context->OutputTextureDatas.Add(FPCGContext::NewObject_AnyThread<UPCGTextureData>(Context));
			}
		}

		for (int32 Index = 0; Index < Context->EffectiveChannelNames.Num(); ++Index)
		{
			UPCGTextureData* TextureData = Context->OutputTextureDatas[Index];
			check(TextureData);

			const int32 Slice = Context->EffectiveSliceIndices[Index];
			if (Slice >= 0)
			{
				bAllInit &= TextureData->Initialize(SourceTexture, /*TextureIndex=*/static_cast<uint32>(Slice), SectionTransform);
			}
			else if (Context->BlackTexture)
			{
				bAllInit &= TextureData->Initialize(Context->BlackTexture, /*TextureIndex=*/0, SectionTransform);
			}
		}
	}

	if (!bAllInit)
	{
		Context->bIsPaused = true;
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
		{
			FPCGContext::FSharedContext<FPCGGetMeshTerrainSectionChannelTexturesContext> SharedContext(ContextHandle);
			if (FPCGGetMeshTerrainSectionChannelTexturesContext* ContextPtr = SharedContext.Get())
			{
				ContextPtr->bIsPaused = false;
			}
		});
		return false;
	}

	// Phase 5: Emit PCG data.
	if (Settings->bOutputTextureArray)
	{
		FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
		Output.Data = Context->OutputTextureArrayData;
	}
	else
	{
		FString TagString;
		for (int32 Index = 0; Index < Context->OutputTextureDatas.Num(); ++Index)
		{
			UPCGTextureData* TextureData = Context->OutputTextureDatas[Index];
			if (!TextureData || !TextureData->IsSuccessfullyInitialized())
			{
				continue;
			}

			FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
			Output.Data = TextureData;

			const FName ChannelName = Context->EffectiveChannelNames[Index];
			if (!ChannelName.IsNone())
			{
				PCGComputeHelpers::GetPrefixedDataLabel(ChannelName.ToString(), TagString);
				Output.Tags.Add(TagString);
			}
		}
	}

	return true;
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
