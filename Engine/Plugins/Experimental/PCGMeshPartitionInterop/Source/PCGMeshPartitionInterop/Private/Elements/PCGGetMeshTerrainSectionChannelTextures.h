// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"

#include "RendererInterface.h"

#include "PCGGetMeshTerrainSectionChannelTextures.generated.h"

class UPCGTexture2DArrayData;
class UPCGTextureData;
class UTexture;
struct IPooledRenderTarget;

namespace UE::MeshPartition
{

/** Emits the input mesh terrain section's baked channel textures. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGGetMeshTerrainSectionChannelTexturesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetMeshTerrainSectionChannelTextures")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~ End UPCGSettings interface

public:
	/** Channel names to operate on. Resolved against the parent mesh partition definition's channel list. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FName> SelectedChannels;

	/** When true, output every channel EXCEPT those in SelectedChannels. When false, output exactly SelectedChannels in user-given order. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bExcludeSelectedChannels = true;

	/** When true, channels not present on this section are dropped from the output, making the output count vary by section. When false (default), missing channels are emitted as zero slices (or black textures in per-channel mode) so the output count and ordering stay stable across sections. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bDropMissingChannels = false;

	/** Emit a single Texture 2D Array (true) or one Texture 2D per selected channel tagged with the channel name (false). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bOutputTextureArray = true;

	/** Filter mode applied when downstream nodes sample the emitted texture(s). */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGTextureFilter Filter = EPCGTextureFilter::Bilinear;
};

struct FPCGGetMeshTerrainSectionChannelTexturesContext : public FPCGContext
{
public:
	virtual ~FPCGGetMeshTerrainSectionChannelTexturesContext();

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

public:
	// Mode A: composited texture array
	TObjectPtr<UPCGTexture2DArrayData> OutputTextureArrayData = nullptr;
	TRefCountPtr<IPooledRenderTarget> CompositedHandle;
	bool bRenderingComplete = false;

	// Mode B: single texture per-channel
	TArray<TObjectPtr<UPCGTextureData>> OutputTextureDatas;

	// Resolved selection. Length is the output array size (or output texture count in mode B).
	TArray<FName> EffectiveChannelNames;
	TArray<int32> EffectiveSliceIndices; // -1 = absent on this section

	TWeakObjectPtr<UTexture> SourceChannelTextureArray;
	TWeakObjectPtr<AActor> SectionActor;

	// Per-channel mode fallback texture for absent channels.
	TObjectPtr<UTexture> BlackTexture;

	// Effective channel/slice arrays populated.
	bool bChannelsResolved = false;

	// Source texture has requested streaming.
	bool bStreamingRequested = false;

	// Snapshot of bForceMiplevelsToBeResident captured before we set it. Restored when execution exits.
	bool bPriorForceMipsResident = false;

	// Source texture is fully streamed in.
	bool bStreamingComplete = false;

	// Time the streaming wait began. Zero if no wait occurred.
	double StreamingWaitStartTime = 0.0;

	// RDG composite render command has been enqueued.
	bool bCompositeScheduled = false;

	// Source array maps 1:1 onto the output; skip the GPU composite.
	bool bUseFastPath = false;
};

class FPCGGetMeshTerrainSectionChannelTexturesElement : public IPCGElementWithCustomContext<FPCGGetMeshTerrainSectionChannelTexturesContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

} // namespace UE::MeshPartition
