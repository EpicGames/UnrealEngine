// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGElement.h"

/**
 * Overrides functions that commonly need to be overriden on an element that will spawn a megamesh
 *  modifier via UE::MeshPartition::Utils::GetPCGManagedMegaMeshModifier.
 */
template <typename SubclassOfIPCGElement>
class TPCGMegaMeshModifierSpawnerElementBase : public SubclassOfIPCGElement
{
public:
	//~ Have to execute on main thread because we spawn a component in the world
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	//~ We can't mark the element as cacheable because even if none of our input data changed, we need a call to 
	//~  PrepareDataInternal to mark our resource as used (done inside the GetPCGManagedMegaMeshModifier call)
	//~  so that it is not deleted. 
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	//~ Deriving classes are expected to handle UPCGPointArrayData as a points type, typically by operating on
	//~  UPCGBasePointData. Otherwise a costly conversion would have to be performed.
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
using FPCGMegaMeshModifierSpawnerElementBase = TPCGMegaMeshModifierSpawnerElementBase<IPCGElement>;
