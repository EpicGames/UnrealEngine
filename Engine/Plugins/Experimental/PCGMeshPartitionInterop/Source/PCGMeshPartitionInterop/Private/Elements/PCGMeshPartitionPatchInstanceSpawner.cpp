// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMeshPartitionPatchInstanceSpawner.h"

#include "PCGModule.h"
#include "PCGComponent.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGHelpers.h"
#include "GameFramework/Actor.h"
#include "EngineUtils.h"
#include "MeshPartition.h"
#include "MeshPartitionPCGUtils.h" // MeshPartition::UPCGManagedModifierResource
#include "PCGMeshPartitionInteropModule.h"

#if WITH_EDITOR
#include "MeshPartitionEditorComponent.h"
#include "Modifiers/MeshPartitionInstancedPatchModifier.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "PCGMegaMeshPatchInstanceSpawner"

namespace UE::MeshPartition
{
namespace PCGMegaMeshPatchInstanceSpawnerLocals
{
#if WITH_EDITOR
	static void InitPatchModifierFromParams(MeshPartition::UInstancedPatchModifier* InPatchModifier, const MeshPartition::FPCGPatchInstanceModifierSpawnerParams& InParams)
	{
		InPatchModifier->SetRadius(InParams.Radius);
		InPatchModifier->SetFalloff(InParams.Falloff);
		InPatchModifier->SetPriority(InParams.Priority);
		InPatchModifier->SetType(InParams.Type);
		InPatchModifier->SetMaxZDistance(InParams.MaxZDistance);
		InPatchModifier->SetWriteToWeightChannel(InParams.bWriteToWeightChannel);
		InPatchModifier->SetWeightChannelName(InParams.WeightChannelName);
	}
#endif // WITH_EDITOR
}

bool FPCGPatchInstanceModifierSpawnerParams::operator==(const MeshPartition::FPCGPatchInstanceModifierSpawnerParams& InOther) const
{
	return Radius == InOther.Radius
		&& Falloff == InOther.Falloff
		&& Priority == InOther.Priority
		&& Type == InOther.Type
		&& MaxZDistance == InOther.MaxZDistance
		&& bWriteToWeightChannel == InOther.bWriteToWeightChannel
		&& WeightChannelName == InOther.WeightChannelName;
}

FPCGElementPtr UPCGPatchInstanceSpawnerSettings::CreateElement() const
{
	return MakeShared<FPCGPatchInstanceSpawnerElement>();
}

bool FPCGPatchInstanceSpawnerElement::PrepareDataInternal(FPCGContext* InContext) const
{
#if !WITH_EDITOR
	return true;
#else
	using namespace PCGMegaMeshPatchInstanceSpawnerLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPatchInstanceSpawnerElement::PrepareDataInternal);
	
	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context != nullptr);

	UPCGComponent* SourceComponent = Cast<UPCGComponent>(InContext->ExecutionSource.Get());
	if (!SourceComponent)
	{
		return true;
	}

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	
	const MeshPartition::UPCGPatchInstanceSpawnerSettings* Settings = Context->GetInputSettings<MeshPartition::UPCGPatchInstanceSpawnerSettings>();
	AActor* TargetActor = Context->GetTypedExecutionTarget<AActor>();
	if (!ensure(Settings && TargetActor))
	{
		return true;
	}

	MeshPartition::Utils::FGetPCGManagedMegaMeshModifierParams ResourceParams;
	ResourceParams.PCGContext = InContext;
	ResourceParams.Element = this;

	// #todo: The affected mega mesh should be passed through the pcg graph or autodetected if none is found.
	// Simply picking the first random one in the map is not a good solution.
	if (auto It = TActorIterator<AMeshPartition>(TargetActor->GetWorld()); It)
	{
		ResourceParams.MegaMesh = *It;
	}

	bool bComponentWasReset = false;
	MeshPartition::UInstancedPatchModifier* Modifier = MeshPartition::Utils::GetPCGManagedMegaMeshModifier<MeshPartition::UInstancedPatchModifier>(ResourceParams, bComponentWasReset);
	if (!ensure(Modifier))
	{
		return true;
	}
	InitPatchModifierFromParams(Modifier, Settings->SpawnerParams);

	const EPCGTimeSliceInitResult ExecutionInitResult = Context->InitializePerExecutionState([Modifier, bComponentWasReset](ContextType* Context, ExecStateType& OutState)
	{
		OutState.Modifier = Modifier;
		OutState.bSkipDueToReuse = !bComponentWasReset;

		OutState.bGenerateOutput = Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel);

		if (OutState.bSkipDueToReuse && !OutState.bGenerateOutput)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		return EPCGTimeSliceInitResult::Success;
	});
	
	if (ExecutionInitResult == EPCGTimeSliceInitResult::NoOperation)
	{
		return true;
	}

	Context->InitializePerIterationStates(Inputs.Num(), [Context, &Inputs, &Outputs](IterStateType& OutState, const ExecStateType& ExecState, int32 Index)
		{
			const MeshPartition::UPCGPatchInstanceSpawnerSettings* Settings = Context->GetInputSettings<MeshPartition::UPCGPatchInstanceSpawnerSettings>();
			check(Settings != nullptr);

			const FPCGTaggedData& Input = Inputs[Index];

			const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data);
			if (!PointData)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("NoPointDataInInput", "Unable to get point data from input"), Context);
				return EPCGTimeSliceInitResult::AbortExecution;
			}

			AActor* TargetActor = Context->GetTypedExecutionTarget<AActor>();
	
			if (!TargetActor)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InvalidTargetActor", "Invalid target actor. Ensure TargetActor member is initialized when creating SpatialData"), Context);
				return EPCGTimeSliceInitResult::AbortExecution;
			}
		
			if (ExecState.bGenerateOutput)
			{
				FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
			}
		
			OutState.TargetActor = TargetActor;
			OutState.PointData = PointData;

			return EPCGTimeSliceInitResult::Success;
		});

	return true;
#endif // WITH_EDITOR
}

bool FPCGPatchInstanceSpawnerElement::ExecuteInternal(FPCGContext* InContext) const
{
#if !WITH_EDITOR
	UE_LOGF(LogPCGMegaMeshInterop, Error, "PCG Patch Spawning is not supported at runtime.");
	return true;
#else
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPatchInstanceSpawnerElement::ExecuteInternal);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context != nullptr);

	if (!Context->DataIsPreparedForExecution())
	{
		return true;
	}

	UPCGComponent* SourceComponent = Cast<UPCGComponent>(InContext->ExecutionSource.Get());
	if (!SourceComponent)
	{
		return true;
	}

	return ExecuteSlice(Context, [SourceComponent](ContextType* Context, const ExecStateType& ExecState, IterStateType& IterState, const uint32 IterIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPatchInstanceSpawnerElement::ExecuteSlice);

			const MeshPartition::UPCGPatchInstanceSpawnerSettings* Settings = Context->GetInputSettings<MeshPartition::UPCGPatchInstanceSpawnerSettings>();
			check(Settings != nullptr);

			if (Context->GetIterationStateResult(IterIndex) == EPCGTimeSliceInitResult::NoOperation)
			{
				return true;
			}
			
			const bool bTargetActorValid = (IterState.TargetActor && IsValid(IterState.TargetActor));
			if (!ExecState.bSkipDueToReuse)
			{
				MeshPartition::UInstancedPatchModifier* PatchModifier = ExecState.Modifier.Get();
				if (!ensure(PatchModifier))
				{
					return true;
				}

				const TConstPCGValueRange<FTransform> TransformRange = IterState.PointData->GetConstTransformValueRange();
				TArray<FVector> InstanceLocations;
				InstanceLocations.Reserve(TransformRange.Num());

				const FTransform ComponentTransform = PatchModifier->GetComponentTransform();

				Algo::Transform(TransformRange, InstanceLocations,
					[&ComponentTransform](const FTransform& PointTransform)
					{
						return ComponentTransform.InverseTransformPosition(PointTransform.GetLocation());
					}
				);
		
				PatchModifier->AddInstances(InstanceLocations);
			}
			return true;
		});
#endif // WITH_EDITOR
}
}

#undef LOCTEXT_NAMESPACE