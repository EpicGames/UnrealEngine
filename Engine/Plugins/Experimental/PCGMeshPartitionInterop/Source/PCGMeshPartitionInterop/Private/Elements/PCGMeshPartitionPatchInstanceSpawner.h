// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "PCGContext.h"
#include "PCGManagedResource.h"
#include "Elements/PCGMeshPartitionModifierSpawnerElementBase.h"
#include "Elements/PCGTimeSlicedElementBase.h"

#include "PCGMeshPartitionPatchInstanceSpawner.generated.h"

class UPCGBasePointData;

namespace UE::MeshPartition
{
class UInstancedPatchModifier;

USTRUCT(BlueprintType, Meta = (DisplayName = "Mesh Partition PCG Patch Instance Spawner Params"))
struct FPCGPatchInstanceModifierSpawnerParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Patch)
	float Radius = 150.f;

	UPROPERTY(EditAnywhere, Category=Patch)
	float Falloff = 150.f;

	UPROPERTY(EditAnywhere, Category=Patch)
	double Priority = 0.;

	UPROPERTY(EditAnywhere, Category=Patch)
	FName Type = TEXT("Patch");

	UPROPERTY(EditAnywhere, Category=Patch)
	float MaxZDistance = 20000.f;

	UPROPERTY(EditAnywhere, Category=Patch)
	bool bWriteToWeightChannel = false;

	UPROPERTY(EditAnywhere, Category=Patch, meta = (EditCondition="bWriteToWeightChannel", EditConditionHides))
	FName WeightChannelName;

	bool operator==(const MeshPartition::FPCGPatchInstanceModifierSpawnerParams& InOther) const;
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGPatchInstanceSpawnerSettings : public UPCGSettings
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	// ~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MegaMeshPatchInstanceSpawner")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMegaMeshPatchInstanceSpawnerSettings", "NodeTitle", "Mesh Partition Patch Instance Spawner"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spawner; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface
	
public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	MeshPartition::FPCGPatchInstanceModifierSpawnerParams SpawnerParams;
};

namespace PCGMegaMeshPatchInstanceSpawner
{
	struct FExecutionContext
	{
		bool bSkipDueToReuse = false;
		bool bGenerateOutput = true;
#if WITH_EDITOR
		TWeakObjectPtr<MeshPartition::UInstancedPatchModifier> Modifier;
#endif
	};
	
	struct FIterationContext
	{
		AActor* TargetActor = nullptr;
		const UPCGBasePointData* PointData = nullptr;
	};
}

class FPCGPatchInstanceSpawnerElement : public TPCGMegaMeshModifierSpawnerElementBase<TPCGTimeSlicedElementBase<
	PCGMegaMeshPatchInstanceSpawner::FExecutionContext, PCGMegaMeshPatchInstanceSpawner::FIterationContext>>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
}
