// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMeshPartitionProjectionSpawner.h"

#include "Data/PCGPointData.h"
#include "Data/PCGDynamicMeshData.h"
#include "Helpers/PCGHelpers.h"
#include "MeshPartition.h"
#include "MeshPartitionPCGUtils.h" // MeshPartition::UPCGManagedModifierResource
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGMeshPartitionInteropModule.h" // LogPCGMegaMeshInterop

#if WITH_EDITOR
#include "MeshPartitionEditorComponent.h"
#include "Modifiers/MeshPartitionInstancedProjectionModifier.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "PCGMegaMeshProjectionSpawner"

namespace UE::MeshPartition
{
namespace PCGMegaMeshMeshProjectSpawnerLocals
{
	// Names of our input pins
	static const FName InMeshPinLabel = TEXT("InMeshes");
	static const FName InMeshTransformsPinLabel = TEXT("InCustomMeshTransforms");

#if WITH_EDITOR
	int32 ChunkSize = 256;
	static FAutoConsoleVariableRef CVarChunkSize(
		TEXT("megamesh.PCGMeshProjection.ChunkSize"),
		ChunkSize,
		TEXT("Execution of the Mesh Partition Projection Instance Spawner node is time sliced by this many points in each data item "
			"(note that it is also always time sliced between data items)."));

#endif // WITH_EDITOR
}

TArray<FPCGPinProperties> UPCGProjectionSpawnerSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& ProjectionsPin = Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	ProjectionsPin.SetRequiredPin();

	FPCGPinProperties& MeshesPin = Properties.Emplace_GetRef(PCGMegaMeshMeshProjectSpawnerLocals::InMeshPinLabel, EPCGDataType::DynamicMesh);
	MeshesPin.SetRequiredPin();
	
#if WITH_EDITOR
	ProjectionsPin.Tooltip = LOCTEXT("ProjectionsPinTooltip", "Raycasts of the projection meshes will be done in the negative Z direction "
		"of each point, and megamesh vertices will be moved along the same axis. The bounds of each point constrain the affected area of the megamesh.");
	MeshesPin.Tooltip = LOCTEXT("MeshesPinTooltip", "Meshes to project to. If there are fewer than number of projections, then they will be cycled through. "
		"By providing a single mesh, all the projections can use the same mesh data, whereas providing the same number of meshes as "
		"projections will result in each projection using a different mesh in a 1:1 manner.");
#endif

	return Properties;
}

FPCGElementPtr UPCGProjectionSpawnerSettings::CreateElement() const
{
	return MakeShared<FPCGProjectionSpawnerElement>();
}

// This sets up our modifier component and gathers the data we will need for execution inside the
//  execution context.
bool FPCGProjectionSpawnerElement::PrepareDataInternal(FPCGContext* InContext) const
{
#if !WITH_EDITOR
	UE_LOGF(LogPCGMegaMeshInterop, Error, "PCG Mesh Partition modifier spawning is not supported at runtime.");
	return true;
#else // WITH_EDITOR

	using namespace PCGMegaMeshMeshProjectSpawnerLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGProjectionSpawnerElement::PrepareDataInternal);
	
	ContextType* Context = static_cast<ContextType*>(InContext);
	if (!ensure(Context))
	{
		return true;
	}
	const MeshPartition::UPCGProjectionSpawnerSettings* Settings = Context->GetInputSettings<MeshPartition::UPCGProjectionSpawnerSettings>();
	if (!ensure(Settings))
	{
		return true;
	}

	if (Settings->AffectedMegaMesh.IsNull())
	{
		//~ TODO: Try finding a megamesh that overlaps the owning PCG component bounds. Note that we would need to adjust our reuse handling
		//~  to make sure that we update the component if this implicit megamesh changes.
		PCGLog::LogErrorOnGraph(LOCTEXT("NoAffectedMegaMesh", "MegaMeshMeshProjectSpawner currently requires "
			"that the AffectedMegamesh be specified (usually via a graph parameter that is routed to an attribute override)."), Context);
		return true;
	}
	if (!Settings->AffectedMegaMesh.IsValid())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("AffectedMegaMeshNotLoaded", "MegaMeshMeshProjectSpawner's affected mesh partition was not loaded."), Context);
		return true;
	}

	UE::MeshPartition::Utils::FGetPCGManagedMegaMeshModifierParams ResourceParams;
	ResourceParams.PCGContext = InContext;
	ResourceParams.Element = this;

	ResourceParams.MegaMesh = Settings->AffectedMegaMesh.Get();
	ResourceParams.Layer = Settings->Type;
	ResourceParams.Priority = Settings->Priority;

	bool bNeedToAddInstances = true;
	MeshPartition::UInstancedProjectionModifier* ModifierComponent = UE::MeshPartition::Utils::GetPCGManagedMegaMeshModifier<MeshPartition::UInstancedProjectionModifier>(ResourceParams, bNeedToAddInstances);
	if (!ensure(ModifierComponent))
	{
		return true;
	}

	ModifierComponent->BlendMode = Settings->BlendMode;
	ModifierComponent->HeightFalloff = Settings->HeightFalloff;
	
	ModifierComponent->WeightChannels.SetNum(Settings->WeightChannels.Num());
	for (int32 EntryIndex = 0; EntryIndex < Settings->WeightChannels.Num(); ++EntryIndex)
	{
		ModifierComponent->WeightChannels[EntryIndex] = Settings->WeightChannels[EntryIndex];
	}

	const EPCGTimeSliceInitResult ExecutionInitResult = Context->InitializePerExecutionState([ModifierComponent, bNeedToAddInstances](ContextType* Context, ExecStateType& OutState)
	{
		// Gather and pre-cast our data
		Algo::Transform(
			Context->InputData.GetInputsByPin(PCGMegaMeshMeshProjectSpawnerLocals::InMeshPinLabel), 
			OutState.MeshInputs,
			[](const FPCGTaggedData& TaggedData) { return Cast<const UPCGDynamicMeshData>(TaggedData.Data); });

		Algo::Transform(
			Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel), 
			OutState.ProjectionTransformInputs,
			[](const FPCGTaggedData& TaggedData) { return Cast<const UPCGBasePointData>(TaggedData.Data); });
		
		// Our types should be guaranteed, but do a check anyway.
		ensure(OutState.MeshInputs.RemoveAll([](const UPCGDynamicMeshData* Data) { return Data == nullptr; }) == 0);
		ensure(OutState.ProjectionTransformInputs.RemoveAll([](const UPCGBasePointData* Data) { return Data == nullptr; }) == 0);

		// Even if we don't have to do any real work, we may still need to pass things down the output pin.
		OutState.bNeedToFillOutputPin = Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel) 
			&& !OutState.ProjectionTransformInputs.IsEmpty();

		if (!bNeedToAddInstances || OutState.MeshInputs.IsEmpty())
		{
			// Nothing more to do if we don't need to modify instances
			return EPCGTimeSliceInitResult::NoOperation;
		}

		OutState.ModifierComponent = ModifierComponent;

		return EPCGTimeSliceInitResult::Success;
	});

	const TArray<const UPCGBasePointData*>& ProjectionTransformInputs = Context->GetPerExecutionState().ProjectionTransformInputs;

	// We want to make an InitializePerIterationStates call even if we aren't doing anything because we would still like to return
	//  true from DataIsPreparedForExecution() so that we can deal with bNeedToFillOutputPin. So we don't return early if the init
	//  result was not a success, we just set the number of iterations to 0 for that call.
	int32 NumIterationStates = ExecutionInitResult == EPCGTimeSliceInitResult::NoOperation ? 0
		: ProjectionTransformInputs.Num();
	
	int32 NumMeshes = Context->GetPerExecutionState().MeshInputs.Num();
	// We should have given a NoOperation result if we had no meshes, but we'll verify here just in case
	if (NumMeshes == 0 && !ensure(NumIterationStates == 0))
	{
		NumIterationStates = 0;
	}

	// The intended use of our node is to either get a 1:1 list of meshes and projection transforms, or a single 
	//  mesh and multiple projection transforms. However, we won't limit usage to that- instead we will iterate
	//  through the projection transforms and iterate circularly through the meshes.
	// We time slice across input data (bundles of points) and within the data by our chunk size. To cycle
	//  properly through meshes across data objects, we prep the mesh starting index for each point bundle/data
	//  ahead of time.

	TArray<int32> MeshStartIndices;
	MeshStartIndices.SetNum(NumIterationStates);
	if (NumIterationStates > 0)
	{
		MeshStartIndices[0] = 0;
	}
	for (int32 i = 1; i < NumIterationStates; ++i)
	{
		const UPCGBasePointData* PreviousPointData = ProjectionTransformInputs[i - 1];
		MeshStartIndices[i] = (MeshStartIndices[i - 1] + PreviousPointData->GetNumPoints()) % NumMeshes;
	}

	Context->InitializePerIterationStates(NumIterationStates, 
		[Context, &MeshStartIndices, NumMeshes, Settings](IterStateType& OutState, const ExecStateType& ExecState, int32 Index)
	{ 
		OutState.NextMeshDataIndex = MeshStartIndices[Index];
		OutState.NextPointIndexInData = 0;

		// Set up the transform attribute accessor for each iteration so that we don't have to recreate or
		//  revalidate it across chunks.
		const UPCGBasePointData* ProjectionData = ExecState.ProjectionTransformInputs[Index];
		if (!ensure(ProjectionData) || ProjectionData->GetNumPoints() == 0
			|| !ensure(OutState.NextPointIndexInData >= 0 && OutState.NextPointIndexInData < ProjectionData->GetNumPoints()))
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}
		FPCGAttributePropertyInputSelector TransformSelector;
		TransformSelector.Update(Settings->CustomMeshTransformsAttribute);
		TUniquePtr<const IPCGAttributeAccessor> TransformAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(ProjectionData, TransformSelector);
		TUniquePtr<const IPCGAttributeAccessorKeys> TransformKeys = PCGAttributeAccessorHelpers::CreateConstKeys(ProjectionData, TransformSelector);
		if (TransformAccessor.IsValid() && TransformKeys.IsValid() && TransformKeys->GetNum() > 0)
		{
			FTransform DummyTransform;
			if (!TransformAccessor->Get<FTransform>(DummyTransform, *TransformKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidTransformAttribute", "Input transform attribute/property '{0}' "
					"is not compatible with a transform."), FText::FromName(TransformSelector.GetName())), Context);
			}
			else
			{
				// Valid, so go ahead and store accessor in iteration state
				OutState.TransformAccessor = MoveTemp(TransformAccessor);
				OutState.TransformKeys = MoveTemp(TransformKeys);
			}
		}

		return EPCGTimeSliceInitResult::Success; 
	});

	return true;
#endif // WITH_EDITOR
}

bool FPCGProjectionSpawnerElement::ExecuteInternal(FPCGContext* InContext) const
{
#if !WITH_EDITOR
	UE_LOGF(LogPCGMegaMeshInterop, Error, "PCG Mesh Partition modifier spawning is not supported at runtime.");
	return true;
#else // WITH_EDITOR

	using namespace PCGMegaMeshMeshProjectSpawnerLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGProjectionSpawnerElement::ExecuteInternal);

	ContextType* Context = static_cast<ContextType*>(InContext);
	if (!ensure(Context) || !Context->DataIsPreparedForExecution())
	{
		return true;
	}

	const MeshPartition::UPCGProjectionSpawnerSettings* Settings = Context->GetInputSettings<MeshPartition::UPCGProjectionSpawnerSettings>();
	if (!ensure(Settings)) { return true; }

	TArray<FPCGTaggedData>& OutputPinData = Context->OutputData.TaggedData;
	if (Context->GetPerExecutionState().bNeedToFillOutputPin && OutputPinData.IsEmpty())
	{
		OutputPinData.Append(Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel));
	}

	bool bAllTimeSlicesDone = ExecuteSlice(Context, [this, Settings](ContextType* Context, const ExecStateType& ExecState, IterStateType& IterState, const uint32 IterIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGProjectionSpawnerElement::ExecuteSlice);

		if (Context->GetIterationStateResult(IterIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		if (!ensure(ExecState.ProjectionTransformInputs.IsValidIndex(IterIndex)
			&& ExecState.MeshInputs.IsValidIndex(IterState.NextMeshDataIndex)))
		{
			return true;
		}

		const UPCGBasePointData* ProjectionData = ExecState.ProjectionTransformInputs[IterIndex];
		if (ProjectionData->GetNumPoints() == 0
			|| !ensure(IterState.NextPointIndexInData >= 0 && IterState.NextPointIndexInData < ProjectionData->GetNumPoints()))
		{
			return true;
		}

		MeshPartition::UInstancedProjectionModifier* ModifierComponent = Context->GetPerExecutionState().ModifierComponent.Get();
		if (!ensure(ModifierComponent)) { return true; }

		int32 ClampedChunkSize = FMath::Clamp(ChunkSize, 1, ProjectionData->GetNumPoints() - IterState.NextPointIndexInData);

		// Custom mesh transforms might be stored on the projection points as an attribute (to allow projecting in a different direction than
		//  the mesh Z). Unpack these for the given chunk.
		TArray<FTransform> ChunkCustomMeshTransforms;
		bool bUseCustomMeshTransforms = IterState.TransformAccessor.IsValid();		if (bUseCustomMeshTransforms && ensure(IterState.TransformKeys))
		{
			ChunkCustomMeshTransforms.SetNumUninitialized(ClampedChunkSize);
			IterState.TransformAccessor->GetRange(TArrayView<FTransform>(ChunkCustomMeshTransforms), IterState.NextPointIndexInData,
				*IterState.TransformKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
		}

		const TConstPCGValueRange<FTransform> TransformRange = ProjectionData->GetConstTransformValueRange();
		const TConstPCGValueRange<FVector> BoundsMinRange = ProjectionData->GetConstBoundsMinValueRange();
		const TConstPCGValueRange<FVector> BoundsMaxRange = ProjectionData->GetConstBoundsMaxValueRange();

		for (int32 i = 0; i < ClampedChunkSize; ++i)
		{
			if (!ensure(IterState.NextPointIndexInData < ProjectionData->GetNumPoints()))
			{
				return true;
			}

			const UPCGDynamicMeshData* DynamicMeshData = ExecState.MeshInputs[IterState.NextMeshDataIndex];
			
			MeshPartition::FInstancedProjectionModifierInstance Instance;
			Instance.Mesh = DynamicMeshData->GetDynamicMesh();
			Instance.ProjectionToWorld = TransformRange[IterState.NextPointIndexInData];
			Instance.ProjectionSpaceBounds = FBox(BoundsMinRange[IterState.NextPointIndexInData], BoundsMaxRange[IterState.NextPointIndexInData]);
			Instance.MeshToWorld = bUseCustomMeshTransforms ? ChunkCustomMeshTransforms[i] : Instance.ProjectionToWorld;

			ModifierComponent->AddInstance(Instance, /*bUpdateCachedMesh*/ false);

			++IterState.NextPointIndexInData;
			IterState.NextMeshDataIndex = (IterState.NextMeshDataIndex + 1) % ExecState.MeshInputs.Num();
		}

		// If our index is past the last point index, we're done with this point bundle (otherwise we need to be called again,
		//  so return false).
		return IterState.NextPointIndexInData >= ProjectionData->GetNumPoints();
	});

	return bAllTimeSlicesDone;
#endif // WITH_EDITOR
}
}

#undef LOCTEXT_NAMESPACE