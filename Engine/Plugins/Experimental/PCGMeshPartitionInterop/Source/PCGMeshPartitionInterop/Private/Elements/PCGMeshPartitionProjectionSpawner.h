// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGMeshPartitionModifierSpawnerElementBase.h"
#include "Elements/PCGTimeSlicedElementBase.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Modifiers/MeshPartitionProjectModifierTypes.h" // MeshPartition::FProjectModifierFalloffSettings etc.

#include "PCGMeshPartitionProjectionSpawner.generated.h"

struct FPCGContext;
class UPCGDynamicMeshData;
class UPCGBasePointData;

namespace UE::MeshPartition
{
class AMeshPartition;
#if WITH_EDITOR
class UInstancedProjectionModifier;
#endif

/**
* Class that defines a PCG node for adding mesh projection modifier instances to a megamesh.
*/
UCLASS(BlueprintType, ClassGroup = (Procedural), Meta = (DisplayName = "Mesh Partition PCG Projection Spawner Settings"))
class UPCGProjectionSpawnerSettings : public UPCGSettings
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	// ~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MegaMeshMeshProjectSpawner")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("MeshPartition::UPCGProjectionSpawnerSettings", "NodeTitle", "Mesh Partition Projection Instance Spawner"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spawner; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface
	
private:

	UPROPERTY(EditAnywhere, Category = Modifier, Meta = (PCG_Overridable))
	double Priority = 0.;

	UPROPERTY(EditAnywhere, Category = Modifier, Meta = (PCG_Overridable))
	FName Type = TEXT("Mesh");

	UPROPERTY(EditAnywhere, Category = Modifier, Meta = (PCG_Overridable))
	TSoftObjectPtr<AMeshPartition> AffectedMegaMesh;

	UPROPERTY(EditAnywhere, Category = Modifier, Meta = (PCG_Overridable))
	MeshPartition::EProjectModifierBlendMode BlendMode = MeshPartition::EProjectModifierBlendMode::Set;

	UPROPERTY(EditAnywhere, Category = Height, Meta = (ShowOnlyInnerProperties))
	MeshPartition::FProjectModifierFalloffSettings HeightFalloff;

	UPROPERTY(EditAnywhere, Category = Channels)
	TArray<FProjectModifierWeightEntry> WeightChannels;

	/**
	* By default, the mesh transforms are the same as the projection transform being used. However, the attribute with
	*  this name can be added to data in the InProjections pin to make meshes have a different transform from the projection. 
	*  This allows for projecting in a direction that does not match the mesh Z axis, but could result in a projection
	*  missing the mesh entirely if the projection bounds don't intersect the mesh at that location.
	*/
	UPROPERTY(EditAnywhere, Category = Modifier, AdvancedDisplay)
	FString CustomMeshTransformsAttribute = TEXT("CustomMeshTransforms");

	friend class FPCGProjectionSpawnerElement;
};

namespace FPCGMegaMeshProjectionSpawner
{
	/**
	* Context that FPCGProjectionSpawnerElement prepares in PrepareDataInternal and holds across
	*  all time slice invocations in ExecuteInternal. This is const inside ExecuteSlice.
	*/
	struct FExecutionContext
	{
		// Whether we need to copy our point inputs to the output pin
		bool bNeedToFillOutputPin = false;

#if WITH_EDITOR
		TWeakObjectPtr<MeshPartition::UInstancedProjectionModifier> ModifierComponent;
#endif

		// We could unpack these each time inside ExecuteInternal, but might as well just carry them over.
		TArray<const UPCGDynamicMeshData*> MeshInputs;
		TArray<const UPCGBasePointData*> ProjectionTransformInputs;
	};

	/**
	* Context held per top-level "iteration", which in this case is per input data object (i.e. bundle of points).
	*/
	struct FIterationContext
	{
		// The next point inside our point data object to process. Updated while iterating so we can resume
		//  in the proper place if we time slice.
		int32 NextPointIndexInData = 0;
		// Index of the next mesh to use while processing our point
		int32 NextMeshDataIndex = 0;

		// Used to 
		TUniquePtr<const IPCGAttributeAccessor> TransformAccessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> TransformKeys;
	};
}

/**
* Class that actually performs spawning of mesh projection instances in PCG
*/
class FPCGProjectionSpawnerElement : public TPCGMegaMeshModifierSpawnerElementBase<TPCGTimeSlicedElementBase<
	MeshPartition::FPCGMegaMeshProjectionSpawner::FExecutionContext, MeshPartition::FPCGMegaMeshProjectionSpawner::FIterationContext>>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
}
