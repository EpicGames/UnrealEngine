// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGMeshPartitionQuery.h"

#include "PCGComponent.h"
#include "PCGSubsystem.h"
#include "Elements/PCGActorSelector.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "Data/PCGPointData.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "MeshPartition.h"
#include "MeshPartitionPCGUtils.h"
#include "Engine/World.h"
#include "PCGMeshPartitionInteropModule.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Async/ParallelFor.h"
#include "MeshPartitionPCGDataComponent.h"
#include "MeshPartitionCompiledSection.h"
#include "UDynamicMesh.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "PCGParamData.h"
#include "PCGSubsystem.h"
#include "MeshPartitionMeshBuilder.h"
#include "MeshPartitionDefinition.h"
#include "Data/PCGMeshPartitionSelectionKey.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "PCGMegaMeshQuery"

namespace PCGMegaMeshQueryLocals
{
	static const FName BoundingShapeLabel = TEXT("Bounding Shape");
}

static TAutoConsoleVariable<bool> CVarEnableBuildCacheResult(TEXT("MegaMesh.PCG.CacheResult"),
	true,
	TEXT("If enabled, pcg builds will be cached locally."));

static TAutoConsoleVariable<bool> CVarEnableBuildDDCWrite(TEXT("MegaMesh.PCG.EnableDDCWrite"),
	false,
	TEXT("If enabled, pcg builds will write built mesh data to ddc."));

static TAutoConsoleVariable<bool> CVarEnableBuildDDCRead(TEXT("MegaMesh.PCG.EnableDDCRead"),
	false,
	TEXT("If enabled, pcg builds will attempt to use DDC to retrieve built mesh data without computing it if possible."));

namespace UE::MeshPartition
{
#if WITH_EDITOR
FText UPCGQuerySettings::GetNodeTooltipText() const
{
	return LOCTEXT("MegaMeshQueryTooltip", "Casts rays from provided points along a given direction until they hit a mesh partition base then transforms the source points to the impact point");
}

void UPCGQuerySettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (IsPropertyOverriddenByPin({ GET_MEMBER_NAME_CHECKED(MeshPartition::UPCGQuerySettings, QueryParams), GET_MEMBER_NAME_CHECKED(MeshPartition::FPCGQueryParams, MegaMeshOverride) }))
	{
		return;
	}

	if (AMeshPartition* SpecifiedMegaMesh = QueryParams.MegaMeshOverride.Get())
	{
		FName LayerName = NAME_None;
		if (QueryParams.QueryType == MeshPartition::EPCGQueryType::Intermediate 
			|| QueryParams.QueryType == MeshPartition::EPCGQueryType::IntermediateLayer)
		{
			if (SpecifiedMegaMesh->GetMeshPartitionDefinition()->GetModifierTypePriorities().Contains(QueryParams.LayerName))
			{
				LayerName = QueryParams.LayerName;
			}
		}

		// Layer specific key
		FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(QueryParams.MegaMeshOverride.ToSoftObjectPath());
		Key.CustomKey.InitializeAs<MeshPartition::FPCGLayerSelectionKey>(QueryParams.QueryType, LayerName, QueryParams.SubPriority, 
			// If we're exclusive, then change must come from lower layer
			!QueryParams.bInclusive,
			EPCGLayerSelectionKeyType::Listener);
		OutKeysToSettings.FindOrAdd(Key).Emplace(this, true);

		// Global change key
		Key.CustomKey.InitializeAs<MeshPartition::FPCGGlobalSelectionKey>();
		OutKeysToSettings.FindOrAdd(Key).Emplace(this, true);
	}
}

void UPCGQuerySettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InOutNode->RenameInputPin(PCGPinConstants::DefaultDependencyOnlyLabel, PCGPinConstants::DefaultExecutionDependencyLabel, /*bInBroadcastUpdate=*/false);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

TArray<FPCGPinProperties> MeshPartition::UPCGQuerySettings::InputPinProperties() const
{
	using namespace PCGMegaMeshQueryLocals;

	TArray<FPCGPinProperties> PinProperties;

	FPCGPinProperties& BoundsPin = PinProperties.Emplace_GetRef(BoundingShapeLabel, EPCGDataType::Spatial,
		/*bInAllowMultipleConnections=*/ false, /*bAllowMultipleData=*/ false,
		LOCTEXT("BoundingShapePinTooltip", "Optional bounds to use instead of the PCG bounds."));
	BoundsPin.SetAdvancedPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGQuerySettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Surface);

	return PinProperties;
}

FPCGElementPtr UPCGQuerySettings::CreateElement() const
{
	return MakeShared<FPCGMeshPartitionQueryElement>();
}

void FPCGMeshPartitionElementContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SurfaceData);
}

bool FPCGMeshPartitionQueryElement::CanExecuteOnlyOnMainThread(FPCGContext* InContext) const
{
	// Without context, we can't know, so force it in the main thread to be safe.
	return !InContext || InContext->CurrentPhase == EPCGExecutionPhase::Execute;
}

bool FPCGMeshPartitionQueryElement::ExecuteInternal(FPCGContext* InContext) const
{
	using namespace PCGMegaMeshQueryLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMeshPartitionQueryElement::Execute);

	FPCGMeshPartitionElementContext* Context = static_cast<FPCGMeshPartitionElementContext*>(InContext);
	const MeshPartition::UPCGQuerySettings* Settings = Context->GetInputSettings<MeshPartition::UPCGQuerySettings>();
	check(Settings);

	MeshPartition::FPCGQueryParams QueryParams = Settings->QueryParams;

	UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	check(SourceComponent);
	UWorld* World = SourceComponent->GetWorld();

	UPCGComponent* OriginalComponent = SourceComponent->GetOriginalComponent();
	AActor* Owner = OriginalComponent ? OriginalComponent->GetOwner() : nullptr;
	const FTransform Transform = Owner ? Owner->GetTransform() : FTransform::Identity;

	FBox Bounds; // Will be grown to contain sections
	FBox InputBounds;
	bool bUnionWasCreated;
	const UPCGSpatialData* BoundingShape = PCGSettingsHelpers::ComputeBoundingShape(Context, BoundingShapeLabel, bUnionWasCreated);
	if (ensure(BoundingShape))
	{
		InputBounds = BoundingShape->GetBounds();
	}
	else
	{
		InputBounds = PCGHelpers::GetGridBounds(Owner, SourceComponent);
	}
	Bounds = InputBounds;
	const FBox LocalBounds = Owner ? Bounds.InverseTransformBy(Owner->GetTransform()) : Bounds;

	if (!QueryParams.bOverrideDefaultParams)
	{
		// Compute default parameters based on original owner component - raycast down local Z axis
		const FVector RayOrigin = Transform.TransformPosition(FVector(0, 0, LocalBounds.Max.Z));
		const FVector RayEnd = Transform.TransformPosition(FVector(0, 0, LocalBounds.Min.Z));

		const FVector::FReal RayLength = (RayEnd - RayOrigin).Length();
		const FVector RayDirection = (RayLength > UE_SMALL_NUMBER ? (RayEnd - RayOrigin) / RayLength : FVector(0, 0, -1.0));

		QueryParams.RayOrigin = RayOrigin;
		QueryParams.RayDirection = RayDirection;
		QueryParams.RayLength = RayLength;
	}
	else // user provided ray parameters
	{
		const FVector::FReal RayDirectionLength = QueryParams.RayDirection.Length();
		if (RayDirectionLength > UE_SMALL_NUMBER)
		{
			QueryParams.RayDirection = QueryParams.RayDirection / RayDirectionLength;
			QueryParams.RayLength *= RayDirectionLength;
		}
		else
		{
			QueryParams.RayDirection = FVector(0, 0, -1.0);
		}
	}
	
	// Before we change bSectionDataGathered and PendingSectionTasks, take note of their starting
	//  state so that we remember to initialize the section data once everything is ready.
	bool bSurfaceDataInitialized = Context->bSectionDataGathered;
#if WITH_EDITOR
	bSurfaceDataInitialized = bSurfaceDataInitialized && Context->PendingSectionTasks.IsEmpty();
#endif

	if (!Context->bSectionDataGathered && Context->NumFramesToWaitBeforeGathering == 0)
	{
		FPCGActorSelectorSettings ActorSelector;
		ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;
		ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
		ActorSelector.bSelectMultiple = true;

		auto BoundsCheck = [](const AActor*)->bool { return true; };
		auto SelfIgnoreCheck = [](const AActor*)->bool { return true; };
		AMeshPartition* SpecifiedMegaMesh = QueryParams.MegaMeshOverride.Get();

		if (!World->IsGameWorld())
		{
#if WITH_EDITOR
			ActorSelector.ActorSelectionClass = AMeshPartition::StaticClass();

			FPCGDynamicTrackingHelper DynamicTracking;
			
			constexpr int32 NumMegaMeshKeys = 2;

			TArray<AActor*> FoundMegaMeshActors;
			if (SpecifiedMegaMesh)
			{
				FoundMegaMeshActors.Add(SpecifiedMegaMesh);

				if (InContext->IsValueOverriden({ GET_MEMBER_NAME_CHECKED(MeshPartition::UPCGQuerySettings, QueryParams), GET_MEMBER_NAME_CHECKED(MeshPartition::FPCGQueryParams, MegaMeshOverride) }))
				{
					DynamicTracking.EnableAndInitialize(InContext, NumMegaMeshKeys);
				}
			} 
			else
			{
				FoundMegaMeshActors = PCGActorSelector::FindActors(ActorSelector, SourceComponent, BoundsCheck, SelfIgnoreCheck);
				DynamicTracking.EnableAndInitialize(InContext, NumMegaMeshKeys * FoundMegaMeshActors.Num());
			}

			auto AddMegaMeshToTracking = [&DynamicTracking, Settings, InContext](AMeshPartition* MegaMesh)
			{
				check(MegaMesh);
				FName LayerName = Settings->QueryParams.LayerName;
				if (Settings->QueryParams.QueryType == MeshPartition::EPCGQueryType::Intermediate)
				{
					if (MegaMesh && MegaMesh->GetMeshPartitionDefinition() && !MegaMesh->GetMeshPartitionDefinition()->GetModifierTypePriorities().Contains(Settings->QueryParams.LayerName))
					{
						LayerName = NAME_None;
					}
				}

				// Create both keys
				FPCGSelectionKey GlobalKey = FPCGSelectionKey::CreateFromObjectPtr(MegaMesh);
				FPCGSelectionKey LayerKey(GlobalKey);
				
				// Specific layer change (with bounds)
				LayerKey.CustomKey.InitializeAs<MeshPartition::FPCGLayerSelectionKey>(Settings->QueryParams.QueryType, 
					LayerName, Settings->QueryParams.SubPriority,
					// If we're exclusive, then change must come from lower layer
					!Settings->QueryParams.bInclusive, 
					EPCGLayerSelectionKeyType::Listener);
				DynamicTracking.AddToTracking(MoveTemp(LayerKey), /*bIsCulled=*/true);

				// Global change (with bounds)
				GlobalKey.CustomKey.InitializeAs<MeshPartition::FPCGGlobalSelectionKey>();
				DynamicTracking.AddToTracking(MoveTemp(GlobalKey), /*bIsCulled=*/true);
			};

			for (AActor* MegaMeshActor : FoundMegaMeshActors)
			{
				if (AMeshPartition* MegaMesh = Cast<AMeshPartition>(MegaMeshActor))
				{
					if (UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent()))
					{
						AddMegaMeshToTracking(MegaMesh);
						
						TArray<MeshPartition::UModifierComponent*> ModifiersToProcess;
						EditorComponent->GetModifiersAffectingBounds(ModifiersToProcess, { InputBounds });

						MeshPartition::FBuilderSettings BuilderSettings;

						BuilderSettings.ModifiersToProcess = MoveTemp(ModifiersToProcess);
						BuilderSettings.Transform = EditorComponent->GetOwner()->GetTransform();
						BuilderSettings.MaxSectionComplexity = EditorComponent->GetPreviewBuildVariant().MaxSectionComplexity;
						BuilderSettings.TypePriorities = EditorComponent->GetMegaMeshDefinition() ? EditorComponent->GetMegaMeshDefinition()->GetModifierTypePriorities() : TArray<FName>{};
						switch (QueryParams.QueryType)
						{
						case MeshPartition::EPCGQueryType::Base:
							BuilderSettings.ModifierFilter = FilterHelpers::FilterOnlyBaseModifiers();
							break;
						case MeshPartition::EPCGQueryType::Intermediate:
							BuilderSettings.ModifierFilter = FilterHelpers::FilterModifiersUntilSubpriorityWithinLayer(
								QueryParams.LayerName, QueryParams.SubPriority, QueryParams.bInclusive);
							break;
						case MeshPartition::EPCGQueryType::IntermediateLayer:
							BuilderSettings.ModifierFilter = FilterHelpers::FilterModifiersByLastLayerToBuild(QueryParams.LayerName, QueryParams.bInclusive);
							break;
						case MeshPartition::EPCGQueryType::Final:
							BuilderSettings.ModifierFilter = [](const FBuilderSettings&, const FModifierDesc&) {return true; };
							break;
						default:
							ensure(false);
						}
						BuilderSettings.bBuildSpatial = true;
						BuilderSettings.bCacheResult = CVarEnableBuildCacheResult.GetValueOnAnyThread();
						BuilderSettings.bAllowDDCRead = CVarEnableBuildDDCRead.GetValueOnAnyThread();
						BuilderSettings.bAllowDDCWrite = CVarEnableBuildDDCWrite.GetValueOnAnyThread();
						BuilderSettings.bRecomputeNormals = QueryParams.QueryType != MeshPartition::EPCGQueryType::Base && QueryParams.bRecomputeVertexNormals;
						BuilderSettings.TexcoordGenerationOptions = FChannelCollectionUVLayoutOptions::GetFromDefinition(EditorComponent->GetMegaMeshDefinition());

						TArray<MeshPartition::FBuildTaskHandle> BuildTaskHandles = Build::LaunchBuilds(BuilderSettings);

						UMeshPartitionEditorSubsystem* Subsystem = UMeshPartitionEditorSubsystem::Get();
						for (MeshPartition::FBuildTaskHandle& TaskHandle : BuildTaskHandles)
						{
							int32 SectionIndex = Context->SectionDatas.Emplace();
							FPCGMeshPartitionElementContext::FSectionData& SectionData = Context->SectionDatas[SectionIndex];
							SectionData.MegaMeshActor = MegaMesh;

							// When we build via the builder, the resulting mesh is in the space of the mesh partition actor.
							SectionData.MeshDataTransform = MegaMeshActor->GetTransform();
								
							if (TaskHandle.IsCompleted())
							{
								SectionData.MeshData = TaskHandle.GetTask()->GetMesh();
								SectionData.Spatial = TaskHandle.GetTask()->GetSpatial();
							}
							else
							{
								Context->PendingSectionTasks.Add(TPair<int32, MeshPartition::FBuildTaskHandle>(SectionIndex, MoveTemp(TaskHandle)));
							}
						}
					}
				}
			}

			DynamicTracking.Finalize(InContext);
#endif // WITH_EDITOR
		}
		else
		{
			if (QueryParams.QueryType == MeshPartition::EPCGQueryType::Final)
			{
				ActorSelector.ActorSelectionClass = MeshPartition::ACompiledSection::StaticClass();

				TArray<AActor*> FoundCompiledSections = PCGActorSelector::FindActors(ActorSelector, SourceComponent, BoundsCheck, SelfIgnoreCheck);
				for (AActor* CompiledSectionActor : FoundCompiledSections)
				{
					if (MeshPartition::ACompiledSection* CompiledSection = Cast<MeshPartition::ACompiledSection>(CompiledSectionActor))
					{
						if (MeshPartition::UPCGDataComponent* PCGData = CompiledSection->GetComponentByClass<MeshPartition::UPCGDataComponent>())
						{
							if (SpecifiedMegaMesh && SpecifiedMegaMesh != CompiledSection->GetParentMegaMesh())
							{
								continue;
							}

							FPCGMeshPartitionElementContext::FSectionData& SectionData = Context->SectionDatas.Emplace_GetRef();
							SectionData.PCGData = PCGData;
							SectionData.MegaMeshActor = CompiledSection->GetParentMegaMesh();
							Bounds += FBox(PCGData->GetSpatial()->GetBoundingBox());
						}
					}
				}
			}
		}

		Context->SurfaceData= FPCGContext::NewObject_AnyThread<MeshPartition::UPCGMeshPartitionData>(Context);
		Context->SurfaceData->QueryParams = QueryParams;
		Context->SurfaceData->OriginatingComponent = SourceComponent;

		Context->bSectionDataGathered = true;
	}

	bool bNeedToWaitForSections = false;
#if WITH_EDITOR
	// See if our builds are completed and remove any from pending if they are
	for (auto Iterator = Context->PendingSectionTasks.CreateIterator(); Iterator; ++Iterator)
	{
		TPair<int32, MeshPartition::FBuildTaskHandle>& PendingSectionTask = *Iterator;
		if (PendingSectionTask.Value.IsCompleted())
		{
			Context->SectionDatas[PendingSectionTask.Key].MeshData = PendingSectionTask.Value.GetTask()->GetMesh();
			Context->SectionDatas[PendingSectionTask.Key].Spatial = PendingSectionTask.Value.GetTask()->GetSpatial();
			Iterator.RemoveCurrentSwap();
		}
	}
	bNeedToWaitForSections = !Context->PendingSectionTasks.IsEmpty();
#endif

	if (Context->bSectionDataGathered && !bSurfaceDataInitialized && !bNeedToWaitForSections)
	{
		Context->SurfaceData->Initialize(Context, World, Transform, Bounds, LocalBounds);
	}

	if (Context->NumFramesToWaitBeforeGathering > 0 || bNeedToWaitForSections || !Context->SurfaceData->IsDataReady())
	{
		Context->bIsPaused = true;
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
			{
				FPCGContext::FSharedContext<FPCGMeshPartitionElementContext> SharedContext(ContextHandle);
				if (FPCGMeshPartitionElementContext* ContextPtr = SharedContext.Get())
				{
					ContextPtr->bIsPaused = false;
					
					if (ContextPtr->NumFramesToWaitBeforeGathering > 0)
					{
						ContextPtr->NumFramesToWaitBeforeGathering--;
					}
				}
			});
		return false;
	}

	FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
	Output.Data = Context->SurfaceData;

	return true;
}
}

#undef LOCTEXT_NAMESPACE