// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "PCGElement.h"
#include "Data/PCGMeshPartitionData.h"

#if WITH_EDITOR
#include "MeshPartitionMeshBuilder.h" // FMeshPartitionBuilderResult
#endif // WITH_EDITOR

#include "PCGMeshPartitionQuery.generated.h"

namespace UE::MeshPartition
{
class AMeshPartition;
class UModifierComponent;
class UPCGDataComponent;

UCLASS(BlueprintType, ClassGroup = (Procedural), Meta = (DisplayName = "Mesh Partition PCG Query Settings"))
class UPCGQuerySettings : public UPCGSettings
{
	GENERATED_BODY()

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MegaMeshQuery")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMegaMeshQuerySettings", "NodeTitle", "Mesh Partition Query"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	MeshPartition::FPCGQueryParams QueryParams;
};

struct FPCGMeshPartitionElementContext : public FPCGContext
{
	struct FSectionData
	{
		// Either the MeshData or PCG data pointer will be valid.
		// if the section data is from a built mega mesh, the PCGData pointer will be filled

		TWeakObjectPtr<MeshPartition::UPCGDataComponent> PCGData;
		
		TSharedPtr<const FMeshData> MeshData;
		TSharedPtr<const FMeshABBTree3> Spatial;
		FTransform MeshDataTransform = FTransform::Identity;

		// Allows us to differentiate data coming from different MegaMeshes
		TWeakObjectPtr<AMeshPartition> MegaMeshActor;
	};

	TArray<FSectionData> SectionDatas;
	// Used to keep track of our context initialization state (specifically, whether we've already kicked off
	//  SectionDatas work) if we take multiple slices to execute the node (due to the build taking time)
	bool bSectionDataGathered = false;

	// @todo: It is possible for the Mesh Query to require a delay between spawning of modifiers and section gathering
	int32 NumFramesToWaitBeforeGathering = 1;
#if WITH_EDITOR
	// If our builds of the mesh don't complete right away, holds handles to the ongoing tasks. The integer in the
	//  pair is an index into SectionDatas.
	TArray<TPair<int32, MeshPartition::FBuildTaskHandle>> PendingSectionTasks;
#endif // WITH_EDITOR

	TObjectPtr<MeshPartition::UPCGMeshPartitionData> SurfaceData = nullptr;

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;
};

class FPCGMeshPartitionQueryElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* InContext) const override;

	virtual FPCGContext* CreateContext() override { return new FPCGMeshPartitionElementContext(); }
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
}