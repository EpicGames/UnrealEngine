// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMeshPartitionSculptLayerWrite.h"

#include "PCGMeshPartitionInteropModule.h" // LogPCGMegaMeshInterop

#if WITH_EDITOR
#include "DynamicMesh/MeshTransforms.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleVariableRef
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "MeshPartition.h"
#include "MeshPartitionPCGUtils.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"
#include "PCGContext.h"
#include "PCGComponent.h"
#include "Spatial/PointHashGrid3.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "PCGMegaMeshSculptLayerWrite"

namespace UE::MeshPartition
{
namespace PCGMegaMeshSculptLayerWriteLocals
{
	static const FName MegaMeshSpatialLabel = TEXT("MegaMeshSpatialLabel");
	static const FName BoundingShapeLabel = TEXT("Bounding Shape");

#if WITH_EDITOR
	
	float VertexSearchCellSize = 0.01f;
	static FAutoConsoleVariableRef CVarVertexSearchCellSize(
		TEXT("MegaMesh.PCGLayerWrite.SearchCellSize"),
		VertexSearchCellSize,
		TEXT("When building the layer write modifier, how far to look for a match in source position. This should be set "
			"fairly low to avoid considering too many vertices, but high enough to be tolerant to roundoff when "
			"converting to world space."));

	void AppendTrianglesThatTouchSourcePositions(bool bRequireAllVertsPerTriangle,
		const FMeshData& SectionMesh, const FTransform& SectionMeshTransform, 
		const Geometry::TPointHashGrid3<int32, double>& HashGrid, TArray<FVector3d> SourcePositions, 
		Geometry::FDynamicMesh3& WorldSpaceProjectionMeshOut)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSculptLayerWriteElement::AppendTrianglesThatTouchSourcePositions);

		// Minor note: We don't currently support time slicing for this element, but if we did, we
		//  could consider using FPCGAsync::AsyncProcessing for the parallel bits.

		// Mark which verts have a corresponding position in SourcePositions
		TArray<bool> VidRelevanceFlags;
		VidRelevanceFlags.SetNumZeroed(SectionMesh.MaxVertexID());
		ParallelFor(SectionMesh.MaxVertexID(), [&SectionMesh, &SectionMeshTransform, &HashGrid, &SourcePositions, &VidRelevanceFlags](int32 Vid)
		{
			if (!SectionMesh.IsVertex(Vid))
			{
				return;
			}
			FVector3d WorldPosition = SectionMeshTransform.TransformPosition(SectionMesh.GetVertex(Vid));
			int32 FoundIndex = HashGrid.FindNearestInRadius(WorldPosition, VertexSearchCellSize, [&WorldPosition, &SourcePositions](int32 CandidateIndex)
			{
				return FVector3d::DistSquared(WorldPosition, SourcePositions[CandidateIndex]);
			}).Key;
			if (SourcePositions.IsValidIndex(FoundIndex))
			{
				VidRelevanceFlags[Vid] = true;
			}
		});

		// Mark triangles associated with these vertices
		TArray<bool> TidRelevanceFlags;
		TidRelevanceFlags.SetNumZeroed(SectionMesh.MaxTriangleID());
		if (bRequireAllVertsPerTriangle)
		{
			ParallelFor(SectionMesh.MaxTriangleID(), [&SectionMesh, &VidRelevanceFlags, &TidRelevanceFlags](int32 Tid)
			{
				if (!SectionMesh.IsTriangle(Tid))
				{
					return;
				}
				Geometry::FIndex3i TriVids = SectionMesh.GetTriangle(Tid);
				for (int SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					if (!VidRelevanceFlags[TriVids[SubIdx]])
					{
						return;
					}
				}
				// If we got to here, all verts were relevant
				TidRelevanceFlags[Tid] = true;
			});
		}
		else // Only require one vert to be modified for the tri to be relevant
		{
			ParallelFor(SectionMesh.MaxTriangleID(), [&SectionMesh, &VidRelevanceFlags, &TidRelevanceFlags](int32 Tid)
			{
				if (!SectionMesh.IsTriangle(Tid))
				{
					return;
				}
				Geometry::FIndex3i TriVids = SectionMesh.GetTriangle(Tid);
				for (int SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					if (VidRelevanceFlags[TriVids[SubIdx]])
					{
						TidRelevanceFlags[Tid] = true;
						return;
					}
				}
			});
		}

		// Append all relevant triangles
		TArray<int32> SourceVidToInsertedVid;
		SourceVidToInsertedVid.Init(IndexConstants::InvalidID, SectionMesh.MaxVertexID());
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSculptLayerWriteElement::AppendTrianglesThatTouchSourcePositions::Append);
			for (int32 Tid : SectionMesh.TriangleIndicesItr())
			{
				if (!TidRelevanceFlags[Tid])
				{
					continue;
				}

				Geometry::FIndex3i TriVids = SectionMesh.GetTriangle(Tid);
				Geometry::FIndex3i RemappedTriVids;
				for (int i = 0; i < 3; ++i)
				{
					int32 SourceVid = TriVids[i];
					if (SourceVidToInsertedVid[SourceVid] < 0)
					{
						FVector3d WorldPosition = SectionMeshTransform.TransformPosition(SectionMesh.GetVertex(SourceVid));
						SourceVidToInsertedVid[SourceVid] = WorldSpaceProjectionMeshOut.AppendVertex(WorldPosition);
					}
					RemappedTriVids[i] = SourceVidToInsertedVid[SourceVid];
				}
				WorldSpaceProjectionMeshOut.AppendTriangle(RemappedTriVids);
			}//end for each tri
		}//end profiling scope
	}//end AppendTrianglesThatTouchSourcePositions


	bool ApplyDataToMeshAttributes(
		const Geometry::TPointHashGrid3<int32, double>& HashGrid, const TArray<FVector3d>& SourcePositions,
		const TOptional<TArray<FVector3d>>& DestinationPositions, TOptional<Geometry::FAxisAlignedBox3d> ConstraintBounds,
		const TArray<TPair<FName, TArray<float>>>& Weights, const FTransform& MeshTransform, Geometry::FDynamicMesh3& MeshOut)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSculptLayerWriteElement::ApplyDataToMeshAttributes);

		MeshOut.DiscardAttributes();
		MeshOut.EnableAttributes();
		
		Geometry::FDynamicMeshAttributeSet* Attributes = MeshOut.Attributes();
		if (!ensure(Attributes))
		{
			return false;
		}

		if (DestinationPositions)
		{
			Attributes->EnableSculptLayers(2);
			if (!ensure(Attributes->GetSculptLayers()))
			{
				return false;
			}
			Attributes->GetSculptLayers()->SetActiveLayer(1);
		}

		Attributes->SetNumWeightLayers(Weights.Num());
		for (int32 WeightLayerIndex = 0; WeightLayerIndex < Attributes->NumWeightLayers(); ++WeightLayerIndex)
		{
			Geometry::FDynamicMeshWeightAttribute* Layer = Attributes->GetWeightLayer(WeightLayerIndex);
			Layer->SetName(Weights[WeightLayerIndex].Key);
		}

		ParallelFor(MeshOut.MaxVertexID(), [&HashGrid, &MeshOut, &SourcePositions, 
			DestPositionsPtr = DestinationPositions.GetPtrOrNull(), &Weights, Attributes, 
			&ConstraintBounds, &MeshTransform](int32 Vid)
		{
			if (!MeshOut.IsVertex(Vid))
			{
				return;
			}
			FVector3d CurrentWorldPosition = MeshTransform.TransformPosition(MeshOut.GetVertex(Vid));
				
			int32 FoundIndex = HashGrid.FindNearestInRadius(CurrentWorldPosition, VertexSearchCellSize, [&CurrentWorldPosition, &SourcePositions](int32 CandidateIndex)
			{
				return FVector3d::DistSquared(CurrentWorldPosition, SourcePositions[CandidateIndex]);
			}).Key;

			if (FoundIndex == IndexConstants::InvalidID || !ensure(SourcePositions.IsValidIndex(FoundIndex)))
			{
				return;
			}

			if (DestPositionsPtr && ensure(DestPositionsPtr->IsValidIndex(FoundIndex)))
			{
				FVector3d Destination = (*DestPositionsPtr)[FoundIndex];
				if (ConstraintBounds.IsSet())
				{
					Destination = ConstraintBounds->Clamp(Destination);
				}
				MeshOut.SetVertex(Vid, MeshTransform.InverseTransformPosition(Destination));
			}

			for (int32 ChannelIndex = 0; ChannelIndex < Weights.Num(); ++ChannelIndex)
			{
				Geometry::FDynamicMeshWeightAttribute* WeightLayer = Attributes->GetWeightLayer(ChannelIndex);
				if (ensure(WeightLayer))
				{
					WeightLayer->SetValue(Vid, &Weights[ChannelIndex].Value[FoundIndex]);
				}
			}
		});// end for vertices in mesh

		if (DestinationPositions)
		{
			Attributes->GetSculptLayers()->UpdateLayersFromMesh();
		}

		return true;
	}//end ApplyDataToMeshAttributes()
#endif //WITH_EDITOR
}//end namespace PCGMegaMeshSculptLayerWriteLocals

TArray<FPCGPinProperties> UPCGSculptLayerWriteSettings::InputPinProperties() const
{
	using namespace PCGMegaMeshSculptLayerWriteLocals;

	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point,
		/*bInAllowMultipleConnections=*/ true, /*bAllowMultipleData=*/ true,
		LOCTEXT("InputPinTooltip", "Points corresponding to vertices. Source and destination world "
			"positions are set via positions or attributes according to node settings, and channel values are float attributes."));
	InputPin.SetRequiredPin();

	FPCGPinProperties& MegaMeshSpatialPin = PinProperties.Emplace_GetRef(PCGMegaMeshSculptLayerWriteLocals::MegaMeshSpatialLabel, 
		EPCGDataType::Spatial, /*bInAllowMultipleConnections=*/ false, /*bAllowMultipleData=*/ false,
		LOCTEXT("MegaMeshSpatialTooltip", "Spatial data for the mesh partition being modified."));
	MegaMeshSpatialPin.SetRequiredPin();

	FPCGPinProperties& BoundsPin = PinProperties.Emplace_GetRef(BoundingShapeLabel, EPCGDataType::Spatial,
		/*bInAllowMultipleConnections=*/ false, /*bAllowMultipleData=*/ false,
		LOCTEXT("BoundingShapePinTooltip", "Optional bounds to use instead of the PCG bounds."));
	BoundsPin.SetAdvancedPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSculptLayerWriteSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DependencyPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
	DependencyPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

FPCGElementPtr UPCGSculptLayerWriteSettings::CreateElement() const
{
	return MakeShared<FPCGSculptLayerWriteElement>();
}

FPCGContext* FPCGSculptLayerWriteElement::CreateContext()
{
	return new FPCGSculptLayerWriteContext();
}

// Does preparation, namely creating a managed modifier if needed
bool FPCGSculptLayerWriteElement::PrepareDataInternal(FPCGContext* InContext) const
{
#if !WITH_EDITOR
	UE_LOGF(LogPCGMegaMeshInterop, Error, "PCG Mesh Partition writing is not supported at runtime.");
	return true;
#else // WITH_EDITOR

	using namespace PCGMegaMeshSculptLayerWriteLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSculptLayerWriteElement::PrepareDataInternal);

	FPCGSculptLayerWriteContext* Context = static_cast<FPCGSculptLayerWriteContext*>(InContext);
	if (!ensure(Context)) 
	{
		return true; 
	}

	const UPCGSculptLayerWriteSettings* Settings = Context->GetInputSettings<UPCGSculptLayerWriteSettings>();
	if (!ensure(Settings)) 
	{ 
		return true; 
	}

	if (!Settings->bWritePositions && Settings->Channels.IsEmpty())
	{
		// Nothing to do
		return true;
	}

	TArray<FPCGTaggedData> MegaMeshSpatialInputs = Context->InputData.GetInputsByPin(PCGMegaMeshSculptLayerWriteLocals::MegaMeshSpatialLabel);

	// Make sure that we have the MegaMesh spatial data
	const MeshPartition::UPCGMeshPartitionData* MegaMeshSpatialData = MegaMeshSpatialInputs.Num() == 1 ?
		Cast<MeshPartition::UPCGMeshPartitionData>(MegaMeshSpatialInputs[0].Data) : nullptr;
	if (!MegaMeshSpatialData)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoMegaMeshData", "FPCGSculptLayerWriteElement requires spatial data from a MeshPartition "
			"query node to build its internal mesh."), Context);
		return true;
	}

	// Get the megamesh to use out of the spatial data
	AMeshPartition* MegaMeshToUse = nullptr;
	for (const MeshPartition::UPCGMeshPartitionData::FSectionData& SectionData : MegaMeshSpatialData->SectionDatas)
	{
		if (!SectionData.MeshData)
		{
			continue;
		}
		
		AMeshPartition* SectionMegaMesh = SectionData.MegaMeshActor.Get();
		if (MegaMeshToUse && MegaMeshToUse != SectionMegaMesh)
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("MultipleMegaMeshes", "Spatial data contains data from more than one MeshPartition actor, but "
				"only one MeshPartition will be targeted."), Context);
			break;
		}
		MegaMeshToUse = SectionMegaMesh;
	}

	if (!MegaMeshToUse)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoAffectedMeshPartition", "Could not find a MeshPartition to target."), Context);
		return true;
	}

	// Prep the modifier that we'll write to
	UE::MeshPartition::Utils::FGetPCGManagedMegaMeshModifierParams ResourceParams;
	ResourceParams.PCGContext = InContext;
	ResourceParams.Element = this;
	ResourceParams.MegaMesh = MegaMeshToUse;
	ResourceParams.Layer = Settings->Type;
	ResourceParams.Priority = Settings->Priority;

	MeshPartition::UProjectMeshLayersModifier* ModifierComponent = UE::MeshPartition::Utils::GetPCGManagedMegaMeshModifier<MeshPartition::UProjectMeshLayersModifier>(
		ResourceParams, Context->bNeedToReinitialize);
	if (!ensure(ModifierComponent))
	{
		return true;
	}

	Context->ModifierComponent = ModifierComponent;

	return true;

#endif // WITH_EDITOR
}

bool FPCGSculptLayerWriteElement::ExecuteInternal(FPCGContext* InContext) const
{
#if !WITH_EDITOR
	UE_LOGF(LogPCGMegaMeshInterop, Error, "PCG Mesh Partition writing is not supported at runtime.");
	return true;
#else // WITH_EDITOR

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSculptLayerWriteElement::Execute);

	using namespace PCGMegaMeshSculptLayerWriteLocals;

	if (!ensure(InContext)) 
	{
		return true; 
	}

	FPCGSculptLayerWriteContext* Context = static_cast<FPCGSculptLayerWriteContext*>(InContext);
	const UPCGSculptLayerWriteSettings* Settings = Context->GetInputSettings<UPCGSculptLayerWriteSettings>();
	if (!ensure(Settings))
	{
		return true;
	}

	if (!Settings->bWritePositions && Settings->Channels.IsEmpty())
	{
		// Nothing to do
		return true;
	}

	MeshPartition::UProjectMeshLayersModifier* ModifierComponent = Context->ModifierComponent.Get();

	if (!ModifierComponent || !Context->bNeedToReinitialize)
	{
		// Input must have either been invalid or not have changed
		return true;
	}

	// This will be used later when extracting data out of our inputs
	UE::MeshPartition::Utils::FGatherNewVertexDataFromPointDataInputParams GatherParams;

	GatherParams.PointInputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (GatherParams.PointInputs.IsEmpty())
	{
		return true;
	}
	
	TArray<FPCGTaggedData> MegaMeshSpatialInputs = Context->InputData.GetInputsByPin(PCGMegaMeshSculptLayerWriteLocals::MegaMeshSpatialLabel);
	const MeshPartition::UPCGMeshPartitionData* MegaMeshSpatialData = MegaMeshSpatialInputs.Num() == 1 ?
		Cast<MeshPartition::UPCGMeshPartitionData>(MegaMeshSpatialInputs[0].Data) : nullptr;
	if (!ensure(MegaMeshSpatialData))
	{
		// This should have been caught in PrepareDataInternal
		return true;
	}

	if (Settings->bSourcePositionIsAttribute)
	{
		GatherParams.SourcePositionsAttribute = Settings->SourcePositionsAttribute;
	}
	GatherParams.ChannelsIn = Settings->Channels;
	
	GatherParams.ContextForLogging = Context;

	// Gather our source and destination points
	TArray<FVector3d> SourcePositions;
	TOptional<TArray<FVector3d>> DestinationPositions;
	TArray<FVector3d>* DestPositionsPtr = nullptr;
	if (Settings->bWritePositions &&
		// Either source or destination should be an attribute, otherwise there is no point in
		//  writing because we would be using the exact same values for both.
		(Settings->bDestinationPositionIsAttribute || Settings->bSourcePositionIsAttribute))
	{
		DestPositionsPtr = &DestinationPositions.Emplace();
		if (Settings->bDestinationPositionIsAttribute)
		{
			GatherParams.DestPositionsAttribute = Settings->DestinationPositionsAttribute;
		}
	}
	TArray<TPair<FName, TArray<float>>> Weights;
	UE::MeshPartition::Utils::GatherNewVertexDataFromPointData(GatherParams, SourcePositions, DestPositionsPtr, Weights);

	if (SourcePositions.IsEmpty())
	{
		return true;
	}
	if (DestPositionsPtr && DestPositionsPtr->Num() != SourcePositions.Num())
	{
		DestinationPositions.Reset();
		DestPositionsPtr = nullptr;
	}

	// Figure out our bounds
	TOptional<Geometry::FAxisAlignedBox3d> InputBounds;
	TOptional<Geometry::FAxisAlignedBox3d> ConstraintBounds;
	bool bUnionWasCreated;
	const UPCGSpatialData* BoundingShape = PCGSettingsHelpers::ComputeBoundingShape(Context, BoundingShapeLabel, bUnionWasCreated);

	if (!Context->InputData.GetInputsByPin(BoundingShapeLabel).IsEmpty() && ensure(BoundingShape))
	{
		InputBounds = BoundingShape->GetBounds();
		if (Settings->bConstrainToBounds && DestPositionsPtr)
		{
			ConstraintBounds = BoundingShape->GetBounds();
		}
	}
	else if (UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get()))
	{
		InputBounds = SourceComponent->GetGridBounds();

		if (Settings->bConstrainToBounds && DestPositionsPtr)
		{
			// For constraints, we don't want to constrain any section from moving into adjacent sections, as long as it's
			//  still inside the overall bounds.
			UPCGComponent* OriginalComponent = SourceComponent ? SourceComponent->GetOriginalComponent() : nullptr;
			AActor* Owner = OriginalComponent ? OriginalComponent->GetOwner() : nullptr;
			if (Owner)
			{
				ConstraintBounds = PCGHelpers::GetActorBounds(Owner);
			}
		}
	}
	
	// Build the hashgrid that we'll need for source vert queries.
	Geometry::TPointHashGrid3<int32, double> HashGrid(VertexSearchCellSize, IndexConstants::InvalidID);
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSculptLayerWriteElement::PopulateHashGrid);

		HashGrid.Reserve(SourcePositions.Num());
		for (int32 Index = 0; Index < SourcePositions.Num(); ++Index)
		{
			// Only keep those points that are within our bounds
			FPCGPoint BoundingShapeSample;
			FBox ContainingBox;
			ContainingBox += SourcePositions[Index];
			if (BoundingShape && !BoundingShape->SamplePoint(FTransform(SourcePositions[Index]), ContainingBox, BoundingShapeSample, nullptr))
			{
				continue;
			}

			HashGrid.InsertPointUnsafe(Index, SourcePositions[Index]);
		}
	}

	// Build up our mesh from the spatial data
	Geometry::FDynamicMesh3 ProjectionMesh;
	for (const MeshPartition::UPCGMeshPartitionData::FSectionData& SectionData : MegaMeshSpatialData->SectionDatas)
	{
		if (!SectionData.MeshData || SectionData.MegaMeshActor.Get() != ModifierComponent->GetAffectedMeshPartition())
		{
			continue;
		}
		
		AppendTrianglesThatTouchSourcePositions(Settings->bOnlyIncludeFullyModifiedTriangles, *SectionData.MeshData, SectionData.Transform,
			HashGrid, SourcePositions, ProjectionMesh);
	}

	FVector3d MeshOrigin = InputBounds.IsSet() ? InputBounds->Center() : ProjectionMesh.GetBounds(true).Center();
	FTransform ProjectionMeshTransform(MeshOrigin);
	MeshTransforms::ApplyTransformInverse(ProjectionMesh, ProjectionMeshTransform);

	// Apply the delta data we have to our projection mesh as sculpt layers and attributes
	ApplyDataToMeshAttributes(HashGrid, SourcePositions, DestinationPositions, ConstraintBounds,
		Weights, ProjectionMeshTransform, ProjectionMesh);

	// Make sure the sculpt modifier knows which channels it is supposed to write. Note that we're using
	//  the channel list that includes names we might have gotten from inputs
	// Also note that this needs to happen before we set the mesh, or else it may discard unsculpted triangles
	//  because it does not know about the weights.
	TArray<FSculptLayerModifierWeightAttributeEntry> ChannelEntries;
	for (const FName& ChannelName : Settings->Channels)
	{
		FSculptLayerModifierWeightAttributeEntry& Entry = ChannelEntries.Emplace_GetRef();
		Entry.ChannelName = ChannelName;
	}
	ModifierComponent->SetChannelEntries(ChannelEntries);

	ModifierComponent->SetSculptLayerMesh(ProjectionMesh, /*bTransact*/ false);
	ModifierComponent->SetWorldTransform(ProjectionMeshTransform);
	ModifierComponent->SetClosestPointProjection(Settings->MaxClosestPointDistance);

	if (InputBounds.IsSet())
	{
		ModifierComponent->SetEditVolumeExtents(InputBounds->Extents());
		
		// TODO: Currently there's not a great way to set specific input bounds for the sculpt modifier
		//  to use because they are driven by mesh bounds and the maximal search distance. Ideally we
		//  would be able to set an override, but we'll have to expose this for the user to optionally
		//  remove/change somehow after running the graph.
		//ModifierComponent->SetInputBoundsOverride(InputBounds.GetPtrOrNull());
	}

	return true;

#endif // WITH_EDITOR
}
}

#undef LOCTEXT_NAMESPACE