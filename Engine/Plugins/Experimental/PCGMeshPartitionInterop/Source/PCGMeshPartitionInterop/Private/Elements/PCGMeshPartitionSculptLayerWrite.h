// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGMeshPartitionData.h" // MeshPartition::EPCGQueryType
#include "Elements/PCGMeshPartitionModifierSpawnerElementBase.h"
#include "Modifiers/MeshPartitionProjectModifierTypes.h" // ESculptLayerProjectMethod
#include "PCGMeshPartitionWrite.h" // EPCGMegaMeshWritePositioningMode
#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGMeshPartitionSculptLayerWrite.generated.h"

namespace UE::MeshPartition
{
class AMeshPartition;
class UProjectMeshLayersModifier;

//~ Note: Properly speaking, it might be better to split the operation of this node into multiple nodes
//~  like this:
//~ 1. Conversion from megamesh bounds to dynamic mesh
//~ 2. Dynamic mesh to point data conversion that preserves channel values
//~ 3. Writer node to dynamic mesh sculpt layers and attributes, which uses the processed point data from 2
//~ 4. MegaMesh sculpt modifier spawner, with setter of contained mesh
//~ 
//~ Instead, we have the below, which is one node that uses processed point data from a megamesh and the
//~  megamesh spatial data to build the sculpt layer modifier. This has the advantage of being a drop-in
//~  replacement for a PCGMegaMeshWrite node, being quicker to get out the door, and possibly better UX,
//~  though that is to be determined. Still, we might want to do the above decomposition.
//~
//~ TODO: If we do keep this node as is, we could consider having a common base class with MeshPartition::USimpleWriteModifier,
//~  as much of the settings and setup is the same.

/**
*
*/
UCLASS(BlueprintType, ClassGroup = (Procedural), Meta = (DisplayName = "Mesh Partition PCG Layer Write Settings"))
class UPCGSculptLayerWriteSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MegaMeshSculptLayerWrite")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMegaMeshSculptWrite", "NodeTitle", "MeshPartition Sculpt Layer Write"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGMegaMeshSculptWrite", "NodeTooltip",
		"Edits vertices in the mesh partition by spawning a Sculpt Layers Modifier whose projection mesh is obtained by taking "
		"vertices at specified positions and moving them. The source positions and destination positions are specified "
		"either by attributes or positions of the passed-in points. Can also optionally write weight channel values, "
		"where the channel name should match a float attribute."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/**
	* Type to use for the modifier that will be making modifications to the mesh
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName Type = TEXT("SculptLayerWrite");

	/**
	* Priority for the modifier that will be making modifications to the mesh
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	double Priority = 0.;

	/** Specifies which channels we will want to write. The points should have these as float attributes. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FName> Channels;

	//~ Automatically turns into a checkbox next to SourcePositionsAttribute if it's the only thing in its edit condition
	UPROPERTY()
	bool bSourcePositionIsAttribute = true;

	/**
	* When the source positions are an attribute, the name of this attribute. It should be a FVector attribute.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, Meta = (EditCondition = "bSourcePositionIsAttribute"))
	FString SourcePositionsAttribute = TEXT("SourcePositions");

	/**
	* When false, destination positions are not written (only channels are)
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bWritePositions = true;

	//~ Automatically turns into a checkbox next to DestinationPositionsAttribute if it's the only thing in its edit condition
	UPROPERTY()
	bool bDestinationPositionIsAttribute = false;

	/**
	* When the destination positions are an attribute, the name of this attribute. It should be a FVector attribute.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, Meta = (EditCondition = "bDestinationPositionIsAttribute"))
	FString DestinationPositionsAttribute = TEXT("DestinationPositions");

	/**
	* When true, any attempts to move points outside the bounds of the owning actor are clamped to the bounds of the actor.
	*  Disabling this will allow editing points outside your working PCG volume, but could cause huge modifier bounds if
	*  you accidentally try to write to nonsensical locations (such as using the wrong attribute accidentally).
	*/
	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bConstrainToBounds = true;

	/**
	* When creating the sculpt layer modifier, whether to only include triangles whose vertices were all written to,
	*  or triangles with any vertices written. Setting this to true can prevent unintentional 0's in weight channel
	*  application from unset vertices around the boundary, but would not include isolated written verts.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bOnlyIncludeFullyModifiedTriangles = true;

	UPROPERTY(EditAnywhere, Category = Settings)
	float MaxClosestPointDistance = 100;
};

struct FPCGSculptLayerWriteContext : public FPCGContext
{
	TWeakObjectPtr<MeshPartition::UProjectMeshLayersModifier> ModifierComponent;
	bool bNeedToReinitialize = true;
};

class FPCGSculptLayerWriteElement : public FPCGMegaMeshModifierSpawnerElementBase
{
protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
}