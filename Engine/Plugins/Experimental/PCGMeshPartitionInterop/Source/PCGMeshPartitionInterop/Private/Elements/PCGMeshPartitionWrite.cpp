// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMeshPartitionWrite.h"

#include "PCGMeshPartitionInteropModule.h" // LogPCGMegaMeshInterop

#if WITH_EDITOR
#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionPCGUtils.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Modifiers/MeshPartitionSimpleWriteModifier.h"
#include "PCGContext.h"
#include "PCGComponent.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "PCGMegaMeshWrite"

namespace UE::MeshPartition
{
namespace PCGMegaMeshWriteLocals
{
	static const FName BoundingShapeLabel = TEXT("Bounding Shape");
}

TArray<FPCGPinProperties> UPCGWriteSettings::InputPinProperties() const
{
	using namespace PCGMegaMeshWriteLocals;

	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point,
		/*bInAllowMultipleConnections=*/ true, /*bAllowMultipleData=*/ true,
		LOCTEXT("InputPinTooltip", "Points corresponding to vertices. Source and destination world "
			"positions are set via positions or attributes according to node settings, and channel values are float attributes."));
	InputPin.SetRequiredPin();

	FPCGPinProperties& BoundsPin = PinProperties.Emplace_GetRef(BoundingShapeLabel, EPCGDataType::Spatial,
		/*bInAllowMultipleConnections=*/ false, /*bAllowMultipleData=*/ false,
		LOCTEXT("BoundingShapePinTooltip", "Optional bounds to use instead of the PCG bounds."));
	BoundsPin.SetAdvancedPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGWriteSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DependencyPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
	DependencyPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

FPCGElementPtr UPCGWriteSettings::CreateElement() const
{
	return MakeShared<FPCGWriteElement>();
}

FPCGContext* FPCGWriteElement::CreateContext()
{
	return new FPCGMegaMeshWriteContext();
}

// Does preparation, namely creating a managed write modifier if needed
bool FPCGWriteElement::PrepareDataInternal(FPCGContext* InContext) const
{
#if !WITH_EDITOR
	UE_LOGF(LogPCGMegaMeshInterop, Error, "PCG Mesh Partition writing is not supported at runtime.");
	return true;
#else // WITH_EDITOR

	using namespace PCGMegaMeshWriteLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWriteElement::PrepareDataInternal);

	FPCGMegaMeshWriteContext* Context = static_cast<FPCGMegaMeshWriteContext*>(InContext);
	if (!ensure(Context)) 
	{
		return true; 
	}

	const MeshPartition::UPCGWriteSettings* Settings = Context->GetInputSettings<MeshPartition::UPCGWriteSettings>();
	UPCGComponent* PCGComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	if (!ensure(Settings && PCGComponent)) 
	{ 
		return true; 
	}

	if (!Settings->bWritePositions && Settings->Channels.IsEmpty())
	{
		// Nothing to do
		return true;
	}

	AMeshPartition* MegaMeshToUse = Settings->AffectedMegaMesh.Get();
	if (!MegaMeshToUse)
	{
		MegaMeshToUse = UE::MeshPartition::Utils::FindClosestMegaMesh(*PCGComponent);
	}

	if (!MegaMeshToUse)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoAffectedMegaMesh", "Could not find a MeshPartition to target."), Context);
		return true;
	}

	MeshPartition::Utils::FGetPCGManagedMegaMeshModifierParams ResourceParams;
	ResourceParams.PCGContext = InContext;
	ResourceParams.Element = this;

	ResourceParams.MegaMesh = MegaMeshToUse;
	ResourceParams.Layer = Settings->Type;
	ResourceParams.Priority = Settings->Priority;

	MeshPartition::USimpleWriteModifier* ModifierComponent = UE::MeshPartition::Utils::GetPCGManagedMegaMeshModifier<MeshPartition::USimpleWriteModifier>(
		ResourceParams, Context->bNeedToReinitialize);
	if (!ensure(ModifierComponent))
	{
		return true;
	}

	Context->ModifierComponent = ModifierComponent;
	
	return true;

#endif // WITH_EDITOR
}

bool FPCGWriteElement::ExecuteInternal(FPCGContext* InContext) const
{
#if !WITH_EDITOR
	UE_LOGF(LogPCGMegaMeshInterop, Error, "PCG Mesh Partition writing is not supported at runtime.");
	return true;
#else // WITH_EDITOR

	using namespace PCGMegaMeshWriteLocals;

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWriteElement::Execute);

	if (!ensure(InContext)) 
	{
		return true; 
	}

	FPCGMegaMeshWriteContext* Context = static_cast<FPCGMegaMeshWriteContext*>(InContext);
	const MeshPartition::UPCGWriteSettings* Settings = Context->GetInputSettings<MeshPartition::UPCGWriteSettings>();
	if (!ensure(Settings))
	{
		return true;
	}

	if (!Settings->bWritePositions && Settings->Channels.IsEmpty())
	{
		// Nothing to do
		return true;
	}

	MeshPartition::USimpleWriteModifier* ModifierComponent = Context->ModifierComponent.Get();

	if (!ModifierComponent || !Context->bNeedToReinitialize)
	{
		// Input must have either been invalid or not have changed
		return true;
	}

	// This will be used later when extracting data out of our inputs
	MeshPartition::Utils::FGatherNewVertexDataFromPointDataInputParams GatherParams;

	GatherParams.PointInputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (GatherParams.PointInputs.IsEmpty())
	{
		return true;
	}

	if (Settings->bSourcePositionIsAttribute)
	{
		GatherParams.SourcePositionsAttribute = Settings->SourcePositionsAttribute;
	}
	GatherParams.ChannelsIn = Settings->Channels;

	GatherParams.ContextForLogging = Context;

	// Gather our source and destination points
	TArray<FVector3d> SourcePositions;
	TOptional<TArray<FVector3d>> DestinationPositions;
	TArray<FVector3d>* DestPositionsPtr = nullptr;
	if (Settings->bWritePositions &&
		// Either source or destination should be an attribute, otherwise there is no point in
		//  writing because we would be using the exact same values for both.
		(Settings->bDestinationPositionIsAttribute || Settings->bSourcePositionIsAttribute))
	{
		DestPositionsPtr = &DestinationPositions.Emplace();
		if (Settings->bDestinationPositionIsAttribute)
		{
			GatherParams.DestPositionsAttribute = Settings->DestinationPositionsAttribute;
		}
	}
	TArray<TPair<FName, TArray<float>>> Weights;
	MeshPartition::Utils::GatherNewVertexDataFromPointData(GatherParams, SourcePositions, DestPositionsPtr, Weights);
	if (SourcePositions.IsEmpty())
	{
		return true;
	}
	if (DestPositionsPtr && DestPositionsPtr->Num() != SourcePositions.Num())
	{
		DestinationPositions.Reset();
		DestPositionsPtr = nullptr;
	}

	// Filter our points based on our bounds
	bool bUnionWasCreated;
	const UPCGSpatialData* BoundingShape = PCGSettingsHelpers::ComputeBoundingShape(Context, BoundingShapeLabel, bUnionWasCreated);
	if (BoundingShape)
	{
		for (int32 Index = 0; Index < SourcePositions.Num(); ++Index)
		{
			FPCGPoint UnusedOutput;
			if (!BoundingShape->SamplePoint(FTransform(SourcePositions[Index]), 
				// Local bounds of our sample. Note that this is not allowed to be an invalid box.
				FBox(FVector::ZeroVector, FVector::ZeroVector), 
				UnusedOutput, nullptr))
			{
				SourcePositions.RemoveAtSwap(Index);

				if (DestPositionsPtr && ensure(DestPositionsPtr->IsValidIndex(Index)))
				{
					DestPositionsPtr->RemoveAtSwap(Index);
				}

				for (TPair<FName, TArray<float>>& Channel : Weights)
				{
					if (ensure(Channel.Value.IsValidIndex(Index)))
					{
						Channel.Value.RemoveAtSwap(Index);
					}
				}

				--Index;
			}
		}
	}//end filtering points via source positions

	// See if we have destination constraint bounds
	TOptional<Geometry::FAxisAlignedBox3d> ConstraintBounds;
	if (Settings->bConstrainToBounds && DestPositionsPtr)
	{
		if (!Context->InputData.GetInputsByPin(BoundingShapeLabel).IsEmpty())
		{
			ConstraintBounds = BoundingShape->GetBounds();
		}
		else
		{
			// We don't want to use grid bounds because we don't want to constrain sections to only editing within their own section. 

			UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
			UPCGComponent* OriginalComponent = SourceComponent ? SourceComponent->GetOriginalComponent() : nullptr;
			AActor* Owner = OriginalComponent ? OriginalComponent->GetOwner() : nullptr;
			if (Owner)
			{
				ConstraintBounds = PCGHelpers::GetActorBounds(Owner);
			}
		}
	}
	if (ConstraintBounds.IsSet() && DestPositionsPtr)
	{
		for (int32 i = 0; i < DestPositionsPtr->Num(); ++i)
		{
			(*DestPositionsPtr)[i] = ConstraintBounds->Clamp((*DestPositionsPtr)[i]);
		}
	}

	ModifierComponent->ReinitializePoints(SourcePositions, DestPositionsPtr, Weights);

	return true;

#endif // WITH_EDITOR
}
}

#undef LOCTEXT_NAMESPACE
