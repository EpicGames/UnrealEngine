// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGMeshPartitionModifierSpawnerElementBase.h"
#include "Data/PCGMeshPartitionData.h" // MeshPartition::EPCGQueryType
#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGMeshPartitionWrite.generated.h"

namespace UE::MeshPartition
{

class AMeshPartition;
class USimpleWriteModifier;

/**
*
*/
UCLASS(BlueprintType, ClassGroup = (Procedural), Meta = (DisplayName = "Mesh Partition PCG Write Settings"))
class UPCGWriteSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MegaMeshWrite")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMegaMeshWrite", "NodeTitle", "MeshPartition Write"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGMegaMeshWrite", "NodeTooltip",
		"Edits vertices in the mesh partition by spawning a modifier that takes vertices at specified positions and moves them. The source "
		"positions and destination positions are specified either by attributes or positions of the passed-in points. Can also "
		"optionally write weight channel values to vertices, where the channel name should match a float attribute."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(EditAnywhere, Category = Settings, Meta = (PCG_Overridable))
	TSoftObjectPtr<AMeshPartition> AffectedMegaMesh;

	/**
	* Type to use for the modifier that will be making modifications to the mesh
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName Type = TEXT("SimpleWrite");

	/**
	* Priority for the modifier that will be making modifications to the mesh
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	double Priority = 0.;

	/** Specifies which channels we will want to write. The points should have these as float attributes. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FName> Channels;

	//~ Automatically turns into a checkbox next to SourcePositionsAttribute if it's the only thing in its edit condition
	UPROPERTY()
	bool bSourcePositionIsAttribute = true;

	/**
	* When the source positions are an attribute, the name of this attribute. It should be a FVector attribute.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, Meta = (EditCondition = "bSourcePositionIsAttribute"))
	FString SourcePositionsAttribute = TEXT("SourcePositions");

	/**
	* When false, destination positions are not written (only channels are)
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bWritePositions = true;

	//~ Automatically turns into a checkbox next to DestinationPositionsAttribute if it's the only thing in its edit condition
	UPROPERTY()
	bool bDestinationPositionIsAttribute = false;

	/**
	* When the destination positions are an attribute, the name of this attribute. It should be a FVector attribute.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, Meta = (EditCondition = "bDestinationPositionIsAttribute"))
	FString DestinationPositionsAttribute = TEXT("DestinationPositions");

	/**
	* When true, any attempts to move points outside the bounds of the owning actor are clamped to the bounds of the actor.
	*  Disabling this will allow editing points outside your working PCG volume, but could cause huge modifier bounds if
	*  you accidentally try to write to nonsensical locations (such as using the wrong attribute accidentally).
	*/
	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bConstrainToBounds = true;
};

struct FPCGMegaMeshWriteContext : public FPCGContext
{
	TWeakObjectPtr<MeshPartition::USimpleWriteModifier> ModifierComponent;
	bool bNeedToReinitialize = true;
};

class FPCGWriteElement : public FPCGMegaMeshModifierSpawnerElementBase
{
protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
}