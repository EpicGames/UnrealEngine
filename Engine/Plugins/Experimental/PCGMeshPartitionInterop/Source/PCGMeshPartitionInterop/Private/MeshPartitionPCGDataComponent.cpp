// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPCGDataComponent.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE::MeshPartition
{
UPCGDataComponent::UPCGDataComponent()
{
	Mesh = MakeShared<FMeshData>();
	Spatial = MakeShared<FMeshABBTree3>();
}

void UPCGDataComponent::SetMesh(FMeshData&& InMesh)
{
	AsyncSpatialBuild.Wait();

	*Mesh = MoveTemp(InMesh);

	StartSpatialAsyncBuild();
}

void UPCGDataComponent::StartSpatialAsyncBuild()
{
	check(AsyncSpatialBuild.IsCompleted());

	AsyncSpatialBuild = Tasks::Launch(TEXT("Build Mesh Partition PCG Data Spatial"),
		[this]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UPCGDataComponent::SetMesh_BuildSpatial);
			Spatial->SetMesh(Mesh.Get(), /* bAutoBuild */ true);
		});
}

void UPCGDataComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Since the custom mesh format could change at any point, serialize a dynamic mesh and convert back to the
	// custom format on load to keep data compatible across builds with different formats and in case the custom
	// mesh format changes the underlying data structures. This is only a problem here and not in other serialized
	// cases of the mega mesh mesh format because it's a uobject and we need to be able to load it. The only other
	// serialized case is in DDC which checks the version key before trying to load the data.
	// #todo: remove this when the mesh format is stable and we no longer can switch between two variants.


	if (Ar.IsSaving())
	{
		Geometry::FDynamicMesh3 SaveMesh;
		Mesh->ConvertToDynamicMesh(SaveMesh);
		Ar << SaveMesh;
	}

	if (Ar.IsLoading())
	{
		Geometry::FDynamicMesh3 SavedMesh;
		Ar << SavedMesh;

		Mesh->Clear();
		Mesh->AppendDynamicMesh(SavedMesh, FTransform::Identity);
	}
}

void UPCGDataComponent::PostLoad()
{
	Super::PostLoad();

	StartSpatialAsyncBuild();
}

void UPCGDataComponent::OnUnregister()
{
	AsyncSpatialBuild.Wait();

	Super::OnUnregister();
}

TSharedPtr<FMeshABBTree3> UPCGDataComponent::GetSpatial()
{
	AsyncSpatialBuild.Wait();

	return Spatial;
}
} // namespace UE::MeshPartition