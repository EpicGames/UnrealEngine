// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPCGUtils.h"

#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGElement.h" // FPCGGetDependenciesCrcParams
#include "PCGSettings.h"

#if WITH_EDITOR
#include "Data/PCGBasePointData.h"
#include "EngineUtils.h" // TActorIterator
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "Modifiers/CodeReusableMeshPartitionModifierInterface.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGParamData.h"
#include "Utils/PCGLogErrors.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "MegaMeshPCGUtils"

namespace UE::MeshPartition
{
namespace MegaMeshPCGUtilsLocals
{
#if WITH_EDITOR
	// Updates affected mesh partition property on a modifier and issues the appropriate update calls
	void UpdateAffectedMesh(MeshPartition::UModifierComponent& ModifierComponent, AMeshPartition* NewActor)
	{
		if (ModifierComponent.GetAffectedMeshPartition() == NewActor)
		{
			return;
		}
		UMeshPartitionEditorComponent* PreviousMegaMesh = ModifierComponent.GetMeshPartitionEditorComponent();
		ModifierComponent.SetAffectedMeshPartition(NewActor);
		if (PreviousMegaMesh)
		{
			PreviousMegaMesh->OnModifierAssigned();
		}
		if (UMeshPartitionEditorComponent* CurrentMegaMesh = ModifierComponent.GetMeshPartitionEditorComponent())
		{
			CurrentMegaMesh->OnModifierAssigned();
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGManagedModifierResource::Initialize(MeshPartition::UModifierComponent* InComponent, FPCGCrc InSettingsCrc)
{
	GeneratedComponent = InComponent;
	SettingsCrc = InSettingsCrc;
	bSupportsComponentReset = Cast<ICodeReusableModifier>(InComponent) != nullptr;
}
#endif

// Called with Release(false) at the start of graph execution to disable our modifier without
//  yet deleting it, in case we are able to reuse it.
bool UPCGManagedModifierResource::Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	bool bWillBeDeleted = Super::Release(bHardRelease, OutActorsToDelete);

#if WITH_EDITOR
	ICodeReusableModifier* ReusableModifier = GetComponent<ICodeReusableModifier>();
	if (!bWillBeDeleted && !bHardRelease && ReusableModifier)
	{
		ReusableModifier->SetDisabledByCode(true);
	}
#endif

	return bWillBeDeleted;
}

// Called by our PCG node if it determines that it can use the same component but will need
//  to edit it. The super call will end up calling ResetComponent().
void UPCGManagedModifierResource::MarkAsUsed()
{
	Super::MarkAsUsed();

#if WITH_EDITOR
	ICodeReusableModifier* ReusableModifier = GetComponent<ICodeReusableModifier>();
	if (ensure(ReusableModifier))
	{
		ReusableModifier->SetDisabledByCode(false);
	}
#endif
}

// Called by our PCG node if it determines that it can use the same component without having
//  to modify it at all.
void UPCGManagedModifierResource::MarkAsReused()
{
	Super::MarkAsReused();

#if WITH_EDITOR
	ICodeReusableModifier* ReusableModifier = GetComponent<ICodeReusableModifier>();
	if (ensure(ReusableModifier))
	{
		ReusableModifier->SetDisabledByCode(false);
	}
#endif
}

// Called by PCG at the end of the graph. It is up to this function to determine whether the
//  modifier is used or not, and to free it if not.
bool UPCGManagedModifierResource::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	// It might seem a litte odd to ask the super if we're unused, but this handles bIsMarkedUnused. UPCGManagedProceduralISMComponent
	//  does the same thing.
	if (Super::ReleaseIfUnused(OutActorsToDelete))
	{
		return true;
	}

#if WITH_EDITOR
	MeshPartition::UModifierComponent* Modifier = GetComponent<MeshPartition::UModifierComponent>();
	ICodeReusableModifier* ReusableModifier = Cast<ICodeReusableModifier>(Modifier);
	if (!Modifier || (ReusableModifier && !ReusableModifier->IsUsed()))
	{
		Release(true, OutActorsToDelete);
		return true;
	}
#endif

	return false;
}

// Called when MarkAsUsed is called to prep the component for reuse with edits
void UPCGManagedModifierResource::ResetComponent()
{
#if WITH_EDITOR
	if (ICodeReusableModifier* ReusableModifier = GetComponent<ICodeReusableModifier>())
	{
		ReusableModifier->ResetForReuse();
	}
#endif
}

#if WITH_EDITOR
MeshPartition::UModifierComponent* Utils::GetPCGManagedMegaMeshModifier(TSubclassOf<MeshPartition::UModifierComponent> ModifierClassIn,
	FGetPCGManagedMegaMeshModifierParams& InParams, bool& bOutModifierWasReset)
{
	using namespace MegaMeshPCGUtilsLocals;

	UClass* ModifierClass = ModifierClassIn.Get();
	if (!InParams.Element || !InParams.PCGContext || !ModifierClass)
	{
		return nullptr;
	}

	AActor* PCGActor = InParams.PCGContext->GetTypedExecutionTarget<AActor>();
	UPCGComponent* PCGComponent = Cast<UPCGComponent>(InParams.PCGContext->ExecutionSource.Get());
	const UPCGSettings* Settings = InParams.PCGContext->GetInputSettings<UPCGSettings>();
	if (!ensure(PCGActor && PCGComponent && Settings))
	{
		return nullptr;
	}

	// The CRC is used to see if we can avoid doing work. It is checked for here, but also stored after
	//  doing work.
	if (!InParams.PCGContext->DependenciesCrc.IsValid())
	{
		InParams.Element->GetDependenciesCrc(FPCGGetDependenciesCrcParams(&InParams.PCGContext->InputData, Settings, 
			InParams.PCGContext->ExecutionSource.Get()), InParams.PCGContext->DependenciesCrc);
	}

	FPCGCrc SettingsCrc = Settings->GetSettingsCrc();
	ensure(SettingsCrc.IsValid());

	MeshPartition::UPCGManagedModifierResource* ResourceToUse = nullptr;
	MeshPartition::UModifierComponent* ModifierComponent = nullptr;
	bOutModifierWasReset = true;

	// If the looked-for class doesn't implement ICodeReusableModifier, we will not be able to reuse
	//  an existing component of that type, because regardless of whether our inputs changed or not, we need
	//  to be able to disable the component via ICodeReusableModifier::SetDisabledByCode at the start
	//  of graph reexecution to avoid it affecting nodes before this one.
	if (ModifierClass->ImplementsInterface(UCodeReusableModifier::StaticClass()))
	{
		// See if we already have a component created for this node. 
		PCGComponent->ForEachManagedResource([&ResourceToUse, &ModifierComponent, ModifierClass, PCGActor, SettingsCrc](UPCGManagedResource* InResource)
		{
			// Early out if already found a match
			if (ResourceToUse)
			{
				return;
			}

			MeshPartition::UPCGManagedModifierResource* CastResource = Cast<MeshPartition::UPCGManagedModifierResource>(InResource);
			if (!CastResource // Wrong type, or... 
				|| !CastResource->CanBeUsed()
				|| CastResource->GetSettingsCrc() != SettingsCrc) // Not unique to this node setup
			{
				return;
			}

			MeshPartition::UModifierComponent* ManagedModifier = CastResource->GetComponent<MeshPartition::UModifierComponent>();

			// Do some sanity checks on the managed component
			if (!ensure(IsValid(ManagedModifier) 
				&& ManagedModifier->GetOwner() == PCGActor 
				&& ManagedModifier->Implements<UCodeReusableModifier>()))
			{
				return;
			}

			// Make sure that the managed component is of the type we want, in case the node requests two different types
			if (!ManagedModifier->IsA(ModifierClass))
			{
				return;
			}

			ResourceToUse = CastResource;
			ModifierComponent = ManagedModifier;
		});
	}

	if (ModifierComponent)
	{
		// See if the modifier requires reset
		if (!ResourceToUse->GetCrc().IsValid()
			|| ResourceToUse->GetCrc() != InParams.PCGContext->DependenciesCrc)
		{
			// Marking as used will clear the modifier via a call to ResetComponent
			ResourceToUse->MarkAsUsed();
		}
		else
		{
			// Marking as reused says that we will reuse the component as is, though we can still edit some settings on it.
			ResourceToUse->MarkAsReused();
			bOutModifierWasReset = false;
		}
	}
	else // If we don't yet have a component created
	{
		// The object creation here is mostly based on PCGMegaMeshPatchInstanceSpawner.cpp
		PCGActor->Modify(!PCGComponent->IsInPreviewMode());
		const EObjectFlags ObjectFlags = (PCGComponent->IsInPreviewMode() ? RF_Transient : RF_NoFlags);
		ModifierComponent = NewObject<MeshPartition::UModifierComponent>(PCGActor,
			ModifierClass, MakeUniqueObjectName(PCGActor, ModifierClass));
		ModifierComponent->RegisterComponent();
		PCGActor->AddInstanceComponent(ModifierComponent);
		ModifierComponent->ComponentTags.Add(PCGComponent->GetFName());
		ModifierComponent->ComponentTags.Add(PCGHelpers::DefaultPCGTag);
		ModifierComponent->AttachToComponent(PCGActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false));

		// Set up a resource to hold it
		ResourceToUse = NewObject<MeshPartition::UPCGManagedModifierResource>(PCGComponent);
		ResourceToUse->Initialize(ModifierComponent, Settings->GetSettingsCrc());
		PCGComponent->AddToManagedResources(ResourceToUse);
	}

	if (!ensure(ModifierComponent && ResourceToUse))
	{
		return nullptr;
	}

	ResourceToUse->SetCrc(InParams.PCGContext->DependenciesCrc);

	// Make sure our modifier is initialized with the optional parameters that we were passed in
	if (InParams.Layer.IsSet() && ModifierComponent->GetType() != *InParams.Layer)
	{
		ModifierComponent->SetType(*InParams.Layer);
	}
	if (InParams.Priority.IsSet() && ModifierComponent->GetPriority() != *InParams.Priority)
	{
		ModifierComponent->SetPriority(*InParams.Priority);
	}
	UpdateAffectedMesh(*ModifierComponent, InParams.MegaMesh);

	return ModifierComponent;
}

bool Utils::GatherNewVertexDataFromPointData(
	const FGatherNewVertexDataFromPointDataInputParams& InputParams,
	TArray<FVector3d>& SourcePositionsOut,
	TArray<FVector3d>* DestinationPositionsOut,
	TArray<TPair<FName, TArray<float>>>& WeightsOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Utils::GatherNewVertexDataFromPointData);

	SourcePositionsOut.Reset();
	if (DestinationPositionsOut)
	{
		DestinationPositionsOut->Reset();
	}
	WeightsOut.Reset();

	FPCGAttributePropertyInputSelector SourcePositionsAttributeSelector;
	FPCGAttributePropertyInputSelector DestPositionsAttributeSelector;

	bool bSourcePostionsNeedReadingFromAttribute = InputParams.SourcePositionsAttribute.IsSet();
	if (bSourcePostionsNeedReadingFromAttribute)
	{
		SourcePositionsAttributeSelector.Update(InputParams.SourcePositionsAttribute.GetValue());
	}
	
	bool bDestPositionsNeedReadingFromAttribute = InputParams.DestPositionsAttribute.IsSet()
		&& ensureMsgf(DestinationPositionsOut, TEXT("Destination positions attribute given, but destination positions not part of output."));
	if (bDestPositionsNeedReadingFromAttribute)
	{
		DestPositionsAttributeSelector.Update(InputParams.DestPositionsAttribute.GetValue());
	}

	// If both are not an attribute, then we'll end up writing the same positions as source
	//  and destination, so we really shouldn't have DestinationPostionsOut
	ensure(!DestinationPositionsOut || bSourcePostionsNeedReadingFromAttribute || bDestPositionsNeedReadingFromAttribute);

	// These keep track of which of the expected weight channels we saw on our input data and which we did not see
	//  in any input object. If there's a mismatch among input object (we saw some in one place and not another),
	//  that might be a problem for the user that we should point out.
	TSet<FName> SeenChannels;
	TSet<FName> UnseenChannels;

	for (const FPCGTaggedData& Input : InputParams.PointInputs)
	{
		const UPCGBasePointData* PointData = Cast<const UPCGBasePointData>(Input.Data);
		if (!ensure(PointData) || PointData->GetNumPoints() == 0)
		{
			continue;
		}

		const TConstPCGValueRange<FTransform> TransformRange = PointData->GetConstTransformValueRange();
		const UPCGMetadata* Metadata = PointData->ConstMetadata();
		if (!Metadata || 
			// Already checked GetNumPoints above but just in case
			!ensure(TransformRange.Num() > 0))
		{
			continue;
		}

		int32 PointsStartIndex = SourcePositionsOut.Num();
		int32 NumPoints = TransformRange.Num();
		int32 NumPointsCumulative = PointsStartIndex + NumPoints;

		auto AppendVectorAttributeToArray = [PointData, ContextForLogging = InputParams.ContextForLogging, NumPoints, PointsStartIndex](
			const FPCGAttributePropertyInputSelector& Selector, TArray<FVector3d>& ValuesOut) -> bool
		{
			TUniquePtr<const IPCGAttributeAccessor> SourcePositionsAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, Selector);
			TUniquePtr<const IPCGAttributeAccessorKeys> SourcePositionsKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, Selector);
			if (!(SourcePositionsAccessor.IsValid() && SourcePositionsKeys.IsValid() && SourcePositionsKeys->GetNum() > 0))
			{
				if (ContextForLogging)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NeedPositionProperty", "Require attribute/property '{0}'."),
						FText::FromName(Selector.GetName())), ContextForLogging);
				}
				return false;
			}

			FVector3d DummyVector;
			if (!SourcePositionsAccessor->Get<FVector3d>(DummyVector, *SourcePositionsKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				if (ContextForLogging)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidPositionAttribute", "Input attribute/property '{0}' "
						"is not compatible with a FVector3d."), FText::FromName(Selector.GetName())), ContextForLogging);
				}
				return false;
			}

			ValuesOut.SetNum(PointsStartIndex + NumPoints);
			bool bSuccess = SourcePositionsAccessor->GetRange(TArrayView<FVector3d>(&ValuesOut[PointsStartIndex], NumPoints), 0,
				*SourcePositionsKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
			if (!bSuccess)
			{
				// Remove the ones we appended if we failed to initialize them
				ValuesOut.SetNum(PointsStartIndex);
			}
			return bSuccess;
		};
		
		// Write source positions
		if (bSourcePostionsNeedReadingFromAttribute)
		{
			if (!AppendVectorAttributeToArray(SourcePositionsAttributeSelector, SourcePositionsOut))
			{
				continue;
			}
		}
		else // if reading from positions
		{
			SourcePositionsOut.Reserve(NumPointsCumulative);
			for (int32 i = 0; i < NumPoints; ++i)
			{
				SourcePositionsOut.Add(TransformRange[i].GetLocation());
			}
		}

		// Write destination positions
		if (DestinationPositionsOut)
		{
			if (bDestPositionsNeedReadingFromAttribute)
			{
				if (!AppendVectorAttributeToArray(DestPositionsAttributeSelector, *DestinationPositionsOut))
				{
					// If we failed to get destinations for this input data, we still need some values here
					//  so that other input data lines up. We write the same positions as our destination.
					DestinationPositionsOut->Append(&SourcePositionsOut[PointsStartIndex], NumPoints);
				}
			}
			else // if reading from positions
			{
				DestinationPositionsOut->Reserve(NumPointsCumulative);
				for (int32 i = 0; i < NumPoints; ++i)
				{
					DestinationPositionsOut->Add(TransformRange[i].GetLocation());
				}
			}
		}

		// Now get and write the weight channels
		FPCGAttributePropertyInputSelector Selector;

		auto GetChannelValuesView = [&WeightsOut, PointsStartIndex, NumPoints, NumPointsCumulative](FName ChannelIn)
		{
			TPair<FName, TArray<float>>* ExistingEntry = WeightsOut.FindByPredicate([&ChannelIn](const TPair<FName, TArray<float>>& Candidate) 
				{ return Candidate.Key == ChannelIn; });
			if (!ExistingEntry)
			{
				ExistingEntry = &WeightsOut.Emplace_GetRef(ChannelIn, TArray<float>());
			}
			ExistingEntry->Value.Reserve(NumPointsCumulative);

			// Make sure we have as many previous values in the channel as we had points originally, 
			//  then make room for the new ones
			ExistingEntry->Value.SetNumZeroed(PointsStartIndex);
			ExistingEntry->Value.SetNum(NumPointsCumulative);

			return TArrayView<float>(&ExistingEntry->Value[PointsStartIndex], NumPoints);
		};

		for (const FName& ChannelName : InputParams.ChannelsIn)
		{
			if (ChannelName.IsNone())
			{
				continue;
			}

			Selector.Update(ChannelName.ToString());
			TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, Selector);
			TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, Selector);

			if (!(Accessor.IsValid() && Keys.IsValid() && Keys->GetNum() > 0))
			{
				UnseenChannels.Add(ChannelName);
				continue;
			}

			TArrayView<float> Values = GetChannelValuesView(ChannelName);
			if (!Accessor->GetRange(Values, 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("WeightStoredAsDouble", "Attribute/property '{0}' could not be converted to float."),
					FText::FromName(Selector.GetName())), InputParams.ContextForLogging);
				for (int32 i = 0; i < Values.Num(); ++i)
				{
					Values[i] = 0;
				}
			}
			SeenChannels.Add(ChannelName);
		}//end going through expected channels
	}// end for FPCGTaggedData

	// See if we had any weight channels that existed in some input data but not in other input data.
	if (InputParams.ContextForLogging)
	{
		TSet<FName> ProblemChannels = SeenChannels.Intersect(UnseenChannels);
		for (const FName& ProblemChannel : ProblemChannels)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("MismatchedAttribute", "Attribute/property '{0}' was present on some PointInputs but not all, "
				"which will result in a default value being set on some points."),
				FText::FromName(ProblemChannel)), InputParams.ContextForLogging);
		}
	}

	return !SourcePositionsOut.IsEmpty();
}

AMeshPartition* Utils::FindClosestMegaMesh(const IPCGGraphExecutionSource& ExecutionSource)
{
	const IPCGGraphExecutionState& ExecutionState = ExecutionSource.GetExecutionState();
	FBox ExecutionSourceBounds = ExecutionState.GetBounds();
	if (!ExecutionSourceBounds.IsValid)
	{
		return nullptr;
	}

	UWorld* World = ExecutionState.GetWorld();
	if (!World)
	{
		return nullptr;
	}

	double ClosestSquaredDistance = TNumericLimits<double>::Max();
	AMeshPartition* ClosestMegaMesh = nullptr;
	for (TActorIterator<AActor> It(World, AMeshPartition::StaticClass()); It; ++It)
	{
		AMeshPartition* MegaMesh = Cast<AMeshPartition>(*It);
		if (!MegaMesh)
		{
			continue;
		}

		const FBox Bounds = MegaMesh->GetComponentsBoundingBox(/*bOnlyCollidingComponents*/ false, /*bIncludeFromChildActors*/ true);
		if (Bounds.Intersect(ExecutionSourceBounds))
		{
			return MegaMesh;
		}
		double SquaredDistance = Bounds.ComputeSquaredDistanceToBox(ExecutionSourceBounds);
		if (SquaredDistance < ClosestSquaredDistance)
		{
			ClosestSquaredDistance = SquaredDistance;
			ClosestMegaMesh = MegaMesh;
		}
	}

	return ClosestMegaMesh;
}
#endif // WITH_EDITOR
} // namespace UE::MeshPartition
#undef LOCTEXT_NAMESPACE