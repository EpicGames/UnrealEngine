// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGManagedResource.h" // UPCGManagedComponent

#include "Misc/Optional.h"
#include "Templates/SubclassOf.h"

#include "MeshPartitionPCGUtils.generated.h"

struct FPCGContext;

#if WITH_EDITOR
struct FPCGTaggedData;
class IPCGGraphExecutionSource;
namespace UE::MeshPartition
{
	class AMeshPartition;
	class UModifierComponent;
}
#endif // WITH_EDITOR

namespace UE::MeshPartition::Utils
{
#if WITH_EDITOR

// Param struct for GetPCGManagedMegaMeshModifier
struct FGetPCGManagedMegaMeshModifierParams
{
	// Required parameters
	FPCGContext* PCGContext = nullptr;
	const IPCGElement* Element = nullptr;

	// Optional parameters: set common settings on a modifier
	AMeshPartition* MegaMesh = nullptr;
	TOptional<FName> Layer;
	TOptional<double> Priority;
};

/**
* Given the settings and element, uses the CRCs of inputs and settings to return a pcg-managed modifier that can be used
*  by a pcg node. If the modifier does not implement IPCGReusableMegaMeshModifier, a new instance will always be created.
*  If it does implement the interface, then an existing instance will be reused for any given settings CRC, and it will
*  either be reset or returned as-is depending on whether the inputs CRC changed or didn't change, respectively.
*/
template <typename ModifierClass>
ModifierClass* GetPCGManagedMegaMeshModifier(FGetPCGManagedMegaMeshModifierParams& InParams, bool& bOutModifierWasReset)
{
	return Cast<ModifierClass>(GetPCGManagedMegaMeshModifier(ModifierClass::StaticClass(), InParams, bOutModifierWasReset));
}
MeshPartition::UModifierComponent* GetPCGManagedMegaMeshModifier(TSubclassOf<MeshPartition::UModifierComponent> ModifierClass, 
	FGetPCGManagedMegaMeshModifierParams& InParams, bool& bOutModifierWasReset);

struct FGatherNewVertexDataFromPointDataInputParams
{
	TArray<FPCGTaggedData> PointInputs;
	TArray<FName> ChannelsIn;
	TOptional<FString> SourcePositionsAttribute;
	TOptional<FString> DestPositionsAttribute;
	FPCGContext* ContextForLogging;
};
/**
* Given pcg data that represents source and destination vertex positions and weights, unpack this data.
*/
bool GatherNewVertexDataFromPointData(
	const FGatherNewVertexDataFromPointDataInputParams& InputParams,
	TArray<FVector3d>& SourcePositionsOut,
	TArray<FVector3d>* DestinationPositionsOut,
	TArray<TPair<FName, TArray<float>>>& WeightsOut);

/**
* Find MegaMesh whose bounds are closest to the execution source's bounds, or the first one
*  whose bounds intersect the execution source bounds.
*/
AMeshPartition* FindClosestMegaMesh(const IPCGGraphExecutionSource& ExecutionSource);
#endif // WITH_EDITOR
}

namespace UE::MeshPartition
{
//~ Note: if this header is moved out of the private folder, this class can be moved into a different header to be
//~  kept private.
/**
* Wrapping class for a modifier managed by a pcg context, used internally by GetPCGManagedMegaMeshModifier().
*/
UCLASS(MinimalAPI, BlueprintType, Meta = (DisplayName = "PCGManaged Mesh Partition Modifier Resource"))
class UPCGManagedModifierResource : public UPCGManagedComponent
{
	GENERATED_BODY()

public:
	// Helper for getting a cast version of the contained component
	template<typename ModifierComponentType>
	ModifierComponentType* GetComponent() const
	{
		return Cast<ModifierComponentType>(GeneratedComponent.Get());
	}

	// These are called by GetPCGMegaMeshModifierResource
#if WITH_EDITOR
	void Initialize(MeshPartition::UModifierComponent* InComponent, FPCGCrc SettingsCrc);
#endif
	FPCGCrc GetSettingsCrc() const { return SettingsCrc; }

	// UPCGManagedComponent
	virtual bool SupportsComponentReset() const override { return bSupportsComponentReset; }
	virtual void ResetComponent() override;

	// UPCGManagedResource
	virtual bool Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	virtual void MarkAsUsed() override;
	virtual void MarkAsReused() override;
	virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;

private:
	UPROPERTY()
	FPCGCrc SettingsCrc;
	UPROPERTY()
	bool bSupportsComponentReset = false;
};
} // namespace UE::MeshPartition