// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGManagedMeshPartitionInstancedProjectionModifier.h"

#include "PCGComponent.h"
#include "Elements/PCGMeshPartitionProjectionSpawner.h"

#if WITH_EDITOR
#include "Modifiers/MeshPartitionInstancedProjectionModifier.h"

namespace UE::MeshPartition
{
// Called with Release(false) at the start of graph execution to disable our modifier without
//  yet deleting it, in case we are able to reuse it.
bool UPCGManagedInstancedProjectionModifier::Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	bool bWillBeDeleted = Super::Release(bHardRelease, OutActorsToDelete);
	if (!bWillBeDeleted && !bHardRelease && GetComponent())
	{
		GetComponent()->SetDisabledByCode(true);
	}
	return bWillBeDeleted;
}

// Called by our PCG node if it determines that it can use the same component but will need
//  to edit it. The super call will end up calling ResetComponent().
void UPCGManagedInstancedProjectionModifier::MarkAsUsed()
{
	Super::MarkAsUsed();

	if (ensure(GetComponent()))
	{
		GetComponent()->SetDisabledByCode(false);
	}
}

// Called by our PCG node if it determines that it can use the same component without having
//  to modify it at all.
void UPCGManagedInstancedProjectionModifier::MarkAsReused()
{
	Super::MarkAsReused();

	if (ensure(GetComponent()))
	{
		GetComponent()->SetDisabledByCode(false);
	}
}

// Called by PCG at the end of the graph. It is up to this function to determine whether the
//  modifier is used or not, and to free it if not.
bool UPCGManagedInstancedProjectionModifier::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	// It might seem a litte odd to ask the super if we're unused, but this handles bIsMarkedUnused. UPCGManagedProceduralISMComponent
	//  does the same thing.
	if (Super::ReleaseIfUnused(OutActorsToDelete) || !GetComponent())
	{
		return true;
	}

	if (GetComponent()->NumInstances() == 0)
	{
		Release(true, OutActorsToDelete);
		return true;
	}

	return false;
}

// Called when MarkAsUsed is called to prep the component for reuse with edits
void UPCGManagedInstancedProjectionModifier::ResetComponent()
{
	if (MeshPartition::UInstancedProjectionModifier* PatchModifier = GetComponent())
	{
		PatchModifier->ClearInstances();
		PatchModifier->OnChanged(PatchModifier->ComputeBounds(), EChangeType::StateChange);
	}
}

UInstancedProjectionModifier* MeshPartition::UPCGManagedInstancedProjectionModifier::GetComponent() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UPCGManagedInstancedProjectionModifier::GetComponent);
	return Cast<MeshPartition::UInstancedProjectionModifier>(GeneratedComponent.Get());
}

void UPCGManagedInstancedProjectionModifier::SetComponent(MeshPartition::UInstancedProjectionModifier* InComponent)
{
	GeneratedComponent = InComponent;
}
} // namespace UE::MeshPartition

#endif // WITH_EDITOR