// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGManagedResource.h"

#include "PCGManagedMeshPartitionInstancedProjectionModifier.generated.h"

namespace UE::MeshPartition
{
class UPCGProjectionSpawnerSettings;
class UInstancedProjectionModifier;

/**
* Class for managing a MeshPartition::UInstancedProjectionModifier spawned by a PCG node. Currently, each
*  MeshPartition::UPCGProjectionSpawnerSettings node gets its own component. Once created, we can reuse
*  the same component when rerunning the node by adjusting its settings and instances.
*/
UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedInstancedProjectionModifier : public UPCGManagedComponent
{
	GENERATED_BODY()
public:

#if WITH_EDITOR
	
	// UPCGManagedComponent
	virtual bool SupportsComponentReset() const override { return true; }
	virtual void ResetComponent() override;
	
	// UPCGManagedResource
	virtual bool Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	virtual void MarkAsUsed() override;
	virtual void MarkAsReused() override;
	virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	
	MeshPartition::UInstancedProjectionModifier* GetComponent() const;
	void SetComponent(MeshPartition::UInstancedProjectionModifier* InComponent);

	uint64 GetSettingsUID() const { return SettingsUID; }
	void SetSettingsUID(uint64 InSettingsUID) { SettingsUID = InSettingsUID; }

protected:
#endif

	// Stores a unique identifier of MeshPartition::UPCGProjectionSpawnerSettings, to make sure that we don't try to reuse
	//  the same resource for another node.
	UPROPERTY(Transient)
	uint64 SettingsUID = -1; // purposefully a value that will never happen in data
};
}