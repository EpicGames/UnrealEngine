// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGManagedMeshPartitionPatchModifier.h"
#include "PCGComponent.h"
#include "Elements/PCGMeshPartitionPatchInstanceSpawner.h"

#if WITH_EDITOR
#include "Modifiers/MeshPartitionInstancedPatchModifier.h"
#endif // WITH_EDITOR

namespace UE::MeshPartition
{
FPCGPatchInstanceModifierDescriptor::FPCGPatchInstanceModifierDescriptor(const MeshPartition::FPCGPatchInstanceModifierSpawnerParams& InParams)
{
	Radius = InParams.Radius;
	Falloff = InParams.Falloff;
	Priority = InParams.Priority;
	Type = InParams.Type;
	MaxZDistance = InParams.MaxZDistance;
	bWriteToWeightChannel = InParams.bWriteToWeightChannel;
	WeightChannelName = InParams.WeightChannelName;
}

bool FPCGPatchInstanceModifierDescriptor::operator==(const MeshPartition::FPCGPatchInstanceModifierDescriptor& InOther) const
{
	return Radius == InOther.Radius
			&& Falloff == InOther.Falloff
			&& Priority == InOther.Priority
			&& Type == InOther.Type
			&& MaxZDistance == InOther.MaxZDistance
			&& bWriteToWeightChannel == InOther.bWriteToWeightChannel
			&& WeightChannelName == InOther.WeightChannelName;
}

#if WITH_EDITOR
void UPCGManagedPatchModifier::ForgetComponent()
{
	if (MeshPartition::UInstancedPatchModifier* PatchModifier = GetComponent())
	{
		PatchModifier->ClearInstances();
		PatchModifier->OnChanged(PatchModifier->ComputeBounds(), UE::MeshPartition::EChangeType::StateChange);
	}

	Super::ForgetComponent();
}

bool UPCGManagedPatchModifier::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	if (Super::ReleaseIfUnused(OutActorsToDelete) || !GetComponent())
	{
		return true;
	}
	else if (GetComponent()->NumInstances() == 0)
	{
		GeneratedComponent->DestroyComponent();
		ForgetComponent();
		return true;
	}
	else
	{
		return false;
	}
}

void UPCGManagedPatchModifier::ResetComponent()
{
	if (MeshPartition::UInstancedPatchModifier* PatchModifier = GetComponent())
	{
		PatchModifier->ClearInstances();
		PatchModifier->OnChanged(PatchModifier->ComputeBounds(), UE::MeshPartition::EChangeType::StateChange);
	}
}

void UPCGManagedPatchModifier::MarkAsReused()
{
	Super::MarkAsReused();

	if (MeshPartition::UInstancedPatchModifier* PatchModifier = GetComponent())
	{
		// clear instances without calling OnChanged yet. The component is being re-used so it's about to receive new data and will trigger an update then.
		PatchModifier->ClearInstances();
	}
}

UInstancedPatchModifier* UPCGManagedPatchModifier::GetComponent() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UPCGManagedPatchModifier::GetComponent);
	return Cast<MeshPartition::UInstancedPatchModifier>(GeneratedComponent.Get());
}

void UPCGManagedPatchModifier::SetComponent(MeshPartition::UInstancedPatchModifier* InComponent)
{
	GeneratedComponent = InComponent;
}
#else

// Mega mesh modifiers are not defined at runtime. This managed resource becomes an empty shell at runtime

void UPCGManagedPatchModifier::ForgetComponent()
{
	Super::ForgetComponent();
}

bool UPCGManagedPatchModifier::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	return Super::ReleaseIfUnused(OutActorsToDelete);
}

void UPCGManagedPatchModifier::ResetComponent()
{
}

void UPCGManagedPatchModifier::MarkAsReused()
{
	Super::MarkAsReused();
}

UInstancedPatchModifier* UPCGManagedPatchModifier::GetComponent() const
{
	return nullptr;
}

void UPCGManagedPatchModifier::SetComponent(MeshPartition::UInstancedPatchModifier* InComponent)
{
}
#endif // WITH_EDITOR
}