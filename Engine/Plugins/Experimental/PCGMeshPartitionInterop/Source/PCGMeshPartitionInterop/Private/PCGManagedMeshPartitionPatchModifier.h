// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGManagedResource.h"

#include "PCGManagedMeshPartitionPatchModifier.generated.h"

namespace UE::MeshPartition
{
struct FPCGPatchInstanceModifierSpawnerParams;
class UInstancedPatchModifier;

USTRUCT()
struct FPCGPatchInstanceModifierDescriptor
{
	GENERATED_BODY()

	FPCGPatchInstanceModifierDescriptor() = default;
	FPCGPatchInstanceModifierDescriptor(const FPCGPatchInstanceModifierSpawnerParams& InParams);

	UPROPERTY()
	float Radius = 1000.f;

	UPROPERTY()
	float Falloff = 1000.f;

	UPROPERTY()
	float MaxZDistance = 20000.f;
	
	UPROPERTY()
	double Priority = 0.;

	UPROPERTY()
	FName Type;

	UPROPERTY()
	bool bWriteToWeightChannel = false;

	UPROPERTY()
	FName WeightChannelName;

	bool operator==(const FPCGPatchInstanceModifierDescriptor& InOther) const;
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedPatchModifier : public UPCGManagedComponent
{
	GENERATED_BODY()
public:
	//~Begin UPCGManagedResource interface
	virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	//~End UPCGManagedResource interface

	//~Begin UPCGManagedComponents interface
	virtual void ResetComponent() override;
	virtual bool SupportsComponentReset() const override { return false; }
	virtual void MarkAsReused() override;
	virtual void ForgetComponent() override;
	//~End UPCGManagedComponents interface

	MeshPartition::UInstancedPatchModifier* GetComponent() const;
	void SetComponent(MeshPartition::UInstancedPatchModifier* InComponent);
	
	void SetDescriptorFromParams(const MeshPartition::FPCGPatchInstanceModifierSpawnerParams& InParams) { Descriptor = MeshPartition::FPCGPatchInstanceModifierDescriptor(InParams); }
	const MeshPartition::FPCGPatchInstanceModifierDescriptor& GetDescriptor() { return Descriptor; }

	uint64 GetSettingsUID() const { return SettingsUID; }
	void SetSettingsUID(uint64 InSettingsUID) { SettingsUID = InSettingsUID; }

protected:
	UPROPERTY(Transient)
	uint64 SettingsUID = -1; // purposefully a value that will never happen in data
	
	UPROPERTY()
	MeshPartition::FPCGPatchInstanceModifierDescriptor Descriptor;
};
}