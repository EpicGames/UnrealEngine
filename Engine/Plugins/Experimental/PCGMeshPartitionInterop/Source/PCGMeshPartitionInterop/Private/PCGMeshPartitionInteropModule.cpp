// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGMeshPartitionInteropModule.h"

#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "MeshPartition.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "Subsystems/PCGSubsystem.h"
#include "Data/PCGMeshPartitionData.h"
#include "Data/PCGMeshPartitionSelectionKey.h"
#include "Engine/World.h"
#endif

DEFINE_LOG_CATEGORY(LogPCGMegaMeshInterop);

namespace UE::MeshPartition
{
IMPLEMENT_MODULE(FPCGMegaMeshInteropModule, PCGMeshPartitionInterop);

#if WITH_EDITOR
TAutoConsoleVariable<bool> CVarEnablePCGMegaMeshInteropEvents(
	TEXT("pcg.MegaMesh.EnableInteropEvents"),
	false,
	TEXT("Enable PCG MeshPartition interop events"));
#endif

void FPCGMegaMeshInteropModule::StartupModule()
{
#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FPCGMegaMeshInteropModule::OnPostEngineInit);
#endif
}

void FPCGMegaMeshInteropModule::ShutdownModule()
{
#if WITH_EDITOR
	if (UMeshPartitionEditorSubsystem* EditorSubsystem = UMeshPartitionEditorSubsystem::Get())
	{
		EditorSubsystem->OnMegaMeshChanged().RemoveAll(this);
	}

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
#endif
}

#if WITH_EDITOR

void FPCGMegaMeshInteropModule::OnPostEngineInit()
{
	if (UMeshPartitionEditorSubsystem* EditorSubsystem = UMeshPartitionEditorSubsystem::Get())
	{
		EditorSubsystem->OnMegaMeshChanged().AddRaw(this, &FPCGMegaMeshInteropModule::OnMegaMeshChanged);
	}
}

void FPCGMegaMeshInteropModule::OnMegaMeshChanged(const UE::MeshPartition::FOnChangedEventInfo& ChangedEventInfo)
{
	if (!CVarEnablePCGMegaMeshInteropEvents.GetValueOnAnyThread())
	{
		return;
	}

	if (!ensure(ChangedEventInfo.ChangedEditorComponent))
	{
		return;
	}

	const AMeshPartition* MegaMesh = ChangedEventInfo.ChangedEditorComponent->GetOwner<AMeshPartition>();
	if (!ensure(MegaMesh))
	{
		return;
	}

	const UMeshPartitionDefinition* Definition = MegaMesh->GetMeshPartitionDefinition();
	if (!Definition)
	{
		return;
	}

	UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(MegaMesh->GetWorld());
	if (!PCGSubsystem)
	{
		return;
	}

	FPCGSelectionKey SelectionKey = FPCGSelectionKey::CreateFromPath(MegaMesh);

	for (const auto& [ModifierPath, ModifierChangeInfos] : ChangedEventInfo.ChangedModifiers)
	{
		const MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(ModifierPath.ResolveObject());
		if (!Modifier)
		{
			continue;
		}

		for (const MeshPartition::FModifierChangeInfo& ModifierChangeInfo : ModifierChangeInfos)
		{
			if (ModifierChangeInfo.ChangeType != UE::MeshPartition::EChangeType::StateChange)
			{
				continue;
			}

			// Check for base layer
			bool bFoundType = false;
			if (ModifierChangeInfo.ModifierDesc.IsBase())
			{
				SelectionKey.CustomKey.InitializeAs<MeshPartition::FPCGLayerSelectionKey>(MeshPartition::EPCGQueryType::Base);
				PCGSubsystem->NotifySelectionKeyChanged(SelectionKey, Modifier, ModifierChangeInfo.ChangedBounds);
				bFoundType = true;
			}

			// Intermediate layers
			for (const FName& ModifierType : Definition->GetModifierTypePriorities())
			{
				bool bFromLowerLayer = bFoundType;
				if (!bFoundType && ModifierType == ModifierChangeInfo.ModifierDesc.Type)
				{
					bFoundType = true;
				}

				// All layers starting from the modifiers type should be considered modified
				if (bFoundType)
				{
					SelectionKey.CustomKey.InitializeAs<MeshPartition::FPCGLayerSelectionKey>(
						MeshPartition::EPCGQueryType::Intermediate, ModifierType, ModifierChangeInfo.ModifierDesc.Priority,
						bFromLowerLayer);
					PCGSubsystem->NotifySelectionKeyChanged(SelectionKey, Modifier, ModifierChangeInfo.ChangedBounds);
				}
			}

			// Layers not part of the type priorities are flagged intermediate with no layer name (misc layer) 
			SelectionKey.CustomKey.InitializeAs<MeshPartition::FPCGLayerSelectionKey>(
				MeshPartition::EPCGQueryType::Intermediate, FName(), 
				// Priority doesn't matter, but mark as previous so it matches regardless of inclusivity
				0, true);
			PCGSubsystem->NotifySelectionKeyChanged(SelectionKey, Modifier, ModifierChangeInfo.ChangedBounds);
			
			// Final layer always gets modified
			SelectionKey.CustomKey.InitializeAs<MeshPartition::FPCGLayerSelectionKey>(MeshPartition::EPCGQueryType::Final);
			PCGSubsystem->NotifySelectionKeyChanged(SelectionKey, Modifier, ModifierChangeInfo.ChangedBounds);
		}
	}

	// If we have a global change, send out a global selection key event
	for (const auto& GlobalChange : ChangedEventInfo.ChangedBounds)
	{
		if (GlobalChange.Key != EChangeType::StateChange)
		{
			continue;
		}

		SelectionKey.CustomKey.InitializeAs<MeshPartition::FPCGGlobalSelectionKey>();
		PCGSubsystem->NotifySelectionKeyChanged(SelectionKey, MegaMesh, GlobalChange.Value.Array());
	}
}

#endif
}