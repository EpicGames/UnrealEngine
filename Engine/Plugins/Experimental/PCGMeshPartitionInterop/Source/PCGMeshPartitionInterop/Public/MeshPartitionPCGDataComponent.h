// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "MeshPartitionMeshData.h"

#include "Tasks/Task.h"

#include "MeshPartitionPCGDataComponent.generated.h"


namespace UE::MeshPartition
{
/**
* Caches mesh data required to sample a built mega mesh section quickly without performing conversions between mesh formats dynamically.
* #todo: This component is an intermediate step, we will eventually have a full caching system that will be able to store spatial trees and mesh data for layers in the procedural modifier stack.
*/
UCLASS(MinimalAPI, Meta = (DisplayName = "Mesh Partition PCG Data Component"))
class UPCGDataComponent : public UActorComponent
{
	GENERATED_BODY()
	
public:
	UPCGDataComponent();
	
	// UObject Implementation
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	// End UObject Implementation
	
	// UActorComponent Implementation
	virtual void OnUnregister() override;
	// End UActorComponent Implementation

	TSharedPtr<const FMeshData> GetMesh() const { return Mesh; }

	PCGMESHPARTITIONINTEROP_API void SetMesh(FMeshData&& InMesh);

	TSharedPtr<FMeshABBTree3> GetSpatial();

	Tasks::FTask GetSpatialBuildTask() const { return AsyncSpatialBuild; }
protected:
	void StartSpatialAsyncBuild();

	TSharedPtr<FMeshData> Mesh;
	
	TSharedPtr<FMeshABBTree3> Spatial;
	
	Tasks::FTask AsyncSpatialBuild;
};
}