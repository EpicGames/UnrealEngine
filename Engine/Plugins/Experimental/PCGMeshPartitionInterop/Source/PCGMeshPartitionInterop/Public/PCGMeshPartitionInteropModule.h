// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPCGMegaMeshInterop, Log, All);

namespace UE::MeshPartition
{
struct FOnChangedEventInfo;

class FPCGMegaMeshInteropModule final : public IModuleInterface
{
public:
	/**
	* Called right after the module DLL has been loaded and the module object has been created
	*/
	virtual void StartupModule();

	/**
	* Called before the module is unloaded, right before the module object is destroyed.
	*/
	virtual void ShutdownModule();

private:

#if WITH_EDITOR
	void OnPostEngineInit();
	void OnMegaMeshChanged(const UE::MeshPartition::FOnChangedEventInfo& ChangedEventInfo);
#endif
};
}