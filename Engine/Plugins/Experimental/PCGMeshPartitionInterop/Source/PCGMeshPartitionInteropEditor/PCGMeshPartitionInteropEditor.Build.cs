// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCGMeshPartitionInteropEditor: ModuleRules
	{
		public PCGMeshPartitionInteropEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
			bAllowUETypesInNamespaces = true;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"EditorSubsystem",
					"Engine",
					"PCG",
					"MeshPartition",
					"MeshPartitionEditor",
					"GeometryCore",
					"GeometryFramework",
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MeshConversion",
					"MeshConversionEngineTypes",
					"PCGMeshPartitionInterop",
					"PCGGeometryScriptInterop",
					"UnrealEd", // GEditor
				}
			);
		}
	}
}