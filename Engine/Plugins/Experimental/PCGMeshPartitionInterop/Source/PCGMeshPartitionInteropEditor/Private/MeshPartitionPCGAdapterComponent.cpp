// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPCGAdapterComponent.h"

#include "MeshPartitionPreviewSection.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionPCGDataComponent.h"
#include "MeshDescription.h"
#include "Engine/StaticMesh.h"
#include "MeshDescriptionToDynamicMesh.h"


namespace UE::MeshPartition
{
UPCGAdapterComponent::UPCGAdapterComponent()
{
}

void UPCGAdapterComponent::PostBuildSectionMesh(AActor* InSection, const MeshPartition::FMeshData& InBuiltMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UPCGAdapterComponent::PostBuildSectionMesh);

	if (MeshPartition::ACompiledSection* CompiledActor = Cast<MeshPartition::ACompiledSection>(InSection))
	{
		CompiledActor->Modify(true);

		MeshPartition::UPCGDataComponent* DataComponent = NewObject<MeshPartition::UPCGDataComponent>(CompiledActor, MeshPartition::UPCGDataComponent::StaticClass(), TEXT("PCGDataComponent"));

		InSection->AddInstanceComponent(DataComponent);
		DataComponent->OnComponentCreated();
		DataComponent->RegisterComponent();
		
		MeshPartition::FMeshData MeshData = InBuiltMesh;

		DataComponent->SetMesh(MoveTemp(MeshData));
	}
}

TArray<FBox> UPCGAdapterComponent::ComputeBounds() const
{
	// Adapter is meant to be used as a global "singleton" instance.
	const FBox AdapterBounds(FVector(-HALF_WORLD_MAX), FVector(HALF_WORLD_MAX));
	return { AdapterBounds };
}
}
