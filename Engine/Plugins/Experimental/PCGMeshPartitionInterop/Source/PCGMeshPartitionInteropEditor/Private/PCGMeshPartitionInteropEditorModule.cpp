// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGMeshPartitionInteropEditorModule.h"

#include "PCGMeshPartitionInteropEditorSettings.h"
#include "Data/PCGMeshTerrainSectionData.h"
#include "Visualizations/PCGMeshTerrainSectionDataVisualization.h"

#include "PCGModule.h"
#include "Data/Registry/PCGDataTypeRegistry.h"
#include "PCGDataVisualizationRegistry.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FPCGMegaMeshInteropEditorModule, PCGMeshPartitionInteropEditor);

DEFINE_LOG_CATEGORY(LogPCGMegaMeshInteropEditor);

void FPCGMegaMeshInteropEditorModule::StartupModule()
{
	RegisterPinColors();
	RegisterDataVisualizations();
}

void FPCGMegaMeshInteropEditorModule::ShutdownModule()
{
	UnregisterDataVisualizations();
	UnregisterPinColors();
}

void FPCGMegaMeshInteropEditorModule::RegisterPinColors()
{
	FPCGDataTypeRegistry& Registry = FPCGModule::GetMutableDataTypeRegistry();
	Registry.RegisterPinColorFunction(UE::MeshPartition::FPCGDataTypeInfoMeshTerrainSection::AsId(), [](const FPCGDataTypeIdentifier&)
	{
		return GetDefault<UPCGMeshPartitionInteropEditorSettings>()->MeshTerrainSectionDataPinColor;
	});
}

void FPCGMegaMeshInteropEditorModule::UnregisterPinColors()
{
	FPCGDataTypeRegistry& Registry = FPCGModule::GetMutableDataTypeRegistry();
	Registry.UnregisterPinColorFunction(UE::MeshPartition::FPCGDataTypeInfoMeshTerrainSection::AsId());
}

void FPCGMegaMeshInteropEditorModule::RegisterDataVisualizations()
{
	FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetMutablePCGDataVisualizationRegistry();
	DataVisRegistry.RegisterPCGDataVisualization(UE::MeshPartition::UPCGMeshTerrainSectionData::StaticClass(), MakeUnique<const FPCGMeshTerrainSectionDataVisualization>());
}

void FPCGMegaMeshInteropEditorModule::UnregisterDataVisualizations()
{
	if (FPCGModule::IsPCGModuleLoaded())
	{
		FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetMutablePCGDataVisualizationRegistry();
		DataVisRegistry.UnregisterPCGDataVisualization(UE::MeshPartition::UPCGMeshTerrainSectionData::StaticClass());
	}
}
