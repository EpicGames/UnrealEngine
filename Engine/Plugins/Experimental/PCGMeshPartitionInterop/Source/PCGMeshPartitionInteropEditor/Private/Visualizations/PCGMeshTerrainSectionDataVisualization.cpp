// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizations/PCGMeshTerrainSectionDataVisualization.h"

#include "Data/PCGMeshTerrainSectionData.h"

#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include "GameFramework/Actor.h"
#include "UObject/SoftObjectPath.h"

#define LOCTEXT_NAMESPACE "PCGMeshTerrainSectionDataVisualization"

FPCGTableVisualizerInfo FPCGMeshTerrainSectionDataVisualization::GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const
{
	using namespace UE::MeshPartition;

	FPCGTableVisualizerInfo Info;
	Info.Data = Data;

	const UPCGMeshTerrainSectionData* SectionData = Cast<UPCGMeshTerrainSectionData>(Data);
	if (!SectionData)
	{
		return Info;
	}

	const FSoftObjectPath SectionActorPath(SectionData->GetSectionActor());

	FPCGTableVisualizerColumnInfo& ColumnInfo = Info.ColumnInfos.Emplace_GetRef();
	ColumnInfo.Id = TEXT("SectionActor");
	ColumnInfo.Label = LOCTEXT("SectionActorLabel", "Section Actor");
	ColumnInfo.Tooltip = LOCTEXT("SectionActorTooltip", "Soft object path to the mesh terrain section actor.");
	ColumnInfo.CellAlignment = EPCGTableVisualizerCellAlignment::Left;
	ColumnInfo.Accessor = MakeShared<FPCGConstantValueAccessor<FSoftObjectPath>>(SectionActorPath);
	ColumnInfo.AccessorKeys = MakeShared<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();

	return Info;
}

#undef LOCTEXT_NAMESPACE
