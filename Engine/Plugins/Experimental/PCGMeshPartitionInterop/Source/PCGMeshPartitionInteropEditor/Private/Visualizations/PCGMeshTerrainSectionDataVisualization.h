// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"

class FPCGMeshTerrainSectionDataVisualization : public IPCGDataVisualization
{
public:
	//~ Begin IPCGDataVisualization interface
	virtual void ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const override {}
	virtual FPCGTableVisualizerInfo GetTableVisualizerInfoWithDomain(const UPCGData* Data, const FPCGMetadataDomainID& DomainID) const override;
	//~ End IPCGDataVisualization interface
};
