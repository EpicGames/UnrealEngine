// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "MeshPartitionModifierComponent.h"

#include "UDynamicMesh.h"

#include "MeshPartitionPCGAdapterComponent.generated.h"

class UDynamicMesh;

namespace UE::MeshPartition
{

/**
* The PCG Adapter modifier spawns a MeshPartition::UPCGDataComponent on the final processed mesh.
* The PCGDataComponent contains a cached FDynamicMesh and FDynamicMeshAABBTree which are used for editor and runtime pcg sampling of the built mesh data
*/
UCLASS(meta=(BlueprintSpawnableComponent, DisplayName = "Mesh Partition PCG Adapter Component"))
class UPCGAdapterComponent : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()
	
public:
	UPCGAdapterComponent();

	// Begin MeshPartition::UModifierComponent Implementation
	virtual void PostBuildSectionMesh(AActor* InSection, const MeshPartition::FMeshData& InBuiltMesh) override;
	virtual TArray<FBox> ComputeBounds() const override;
	// End MeshPartition::UModifierComponent Implementation
protected:
};
}
#endif // WITH_EDITOR