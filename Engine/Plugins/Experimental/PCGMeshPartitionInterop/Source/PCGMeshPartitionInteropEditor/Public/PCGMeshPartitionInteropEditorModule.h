// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPCGMegaMeshInteropEditor, Log, All);

class FPCGMegaMeshInteropEditorModule final : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void RegisterPinColors();
	void UnregisterPinColors();

	void RegisterDataVisualizations();
	void UnregisterDataVisualizations();
};