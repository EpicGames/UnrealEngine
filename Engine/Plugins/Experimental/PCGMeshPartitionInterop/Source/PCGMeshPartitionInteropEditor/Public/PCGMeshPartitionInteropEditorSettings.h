// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "PCGMeshPartitionInteropEditorSettings.generated.h"

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta = (DisplayName = "PCG Mesh Partition Interop Editor Settings"))
class UPCGMeshPartitionInteropEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Color used for data pins of type Mesh Terrain Section */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor MeshTerrainSectionDataPinColor = FLinearColor(0.75f, 0.55f, 0.35f);
};
